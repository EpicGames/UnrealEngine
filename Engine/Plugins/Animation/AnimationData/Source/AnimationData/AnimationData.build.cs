// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnimationData : ModuleRules
{
    public AnimationData(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "Engine",
                "MovieScene",
                "MovieSceneTracks",
                "Sequencer", 
                "AnimGraph",
                "ControlRig",
                "ControlRigDeveloper",
                "UnrealEd",
                "AnimationDataController",
                "AnimationCore",
                "AnimationBlueprintLibrary",
				"AnimationEditor",
				"TimeManagement"
            });

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "AnimGraphRuntime",   
                "MovieSceneTools",
            }
        );
    }
}
