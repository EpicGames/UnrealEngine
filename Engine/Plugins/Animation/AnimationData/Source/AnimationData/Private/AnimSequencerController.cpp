// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequencerController.h"
#include "AnimDataControllerActions.h"
#include "CommonFrameRates.h"
#include "ControlRigObjectBinding.h"

#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/AnimData/CurveIdentifier.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimSequence.h"

#include "Algo/Transform.h"
#include "UObject/NameTypes.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Math/UnrealMathUtility.h"
#include "Rigs/FKControlRig.h"

#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"

#include "AnimSequencerHelpers.h"
#include "Animation/AnimationSettings.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequencerController)

#define LOCTEXT_NAMESPACE "AnimSequencerController"

void UAnimSequencerController::SetModel(TScriptInterface<IAnimationDataModel> InModel)
{	
	if (Model != nullptr)
	{
		Model->GetModifiedEvent().RemoveAll(this);
	}

	ModelInterface = InModel;
	Model = CastChecked<UAnimationSequencerDataModel>(InModel.GetObject(), ECastCheckedType::NullAllowed);
	
	ChangeTransactor.SetTransactionObject(Model.Get());
}

void UAnimSequencerController::OpenBracket(const FText& InTitle, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (BracketDepth == 0)
	{
		bShouldTransactBracket = bShouldTransact;
	}

	if (UE::FChangeTransactor::CanTransactChanges() && !ChangeTransactor.IsTransactionPending() && ShouldTransact(bShouldTransact))
	{
		ChangeTransactor.OpenTransaction(InTitle);

		ConditionalAction<UE::Anim::FCloseBracketAction>(bShouldTransact,  InTitle.ToString());
	}

	if (BracketDepth == 0)
	{
		FBracketPayload Payload;
		Payload.Description = InTitle.ToString();

		Model->LockEvaluationAndModification();

		Model->GetNotifier().Notify(EAnimDataModelNotifyType::BracketOpened, Payload);
	}

	++BracketDepth;
}

void UAnimSequencerController::CloseBracket(bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (BracketDepth == 0)
	{
		ReportError(LOCTEXT("NoExistingBracketError", "Attempt to close bracket that was not previously opened"));
		return;
	}

	--BracketDepth;

	if (BracketDepth == 0)
	{
		if (UE::FChangeTransactor::CanTransactChanges() && ShouldTransact(bShouldTransact && bShouldTransactBracket))
		{
			ensure(ChangeTransactor.IsTransactionPending());

			ConditionalAction<UE::Anim::FOpenBracketAction>(bShouldTransact,  TEXT("Open Bracket"));

			ChangeTransactor.CloseTransaction();
		}
		
		Model->GetNotifier().Notify(EAnimDataModelNotifyType::BracketClosed);
		
		Model->UnlockEvaluationAndModification();

		bShouldTransactBracket = false;
	}
}

void UAnimSequencerController::SetNumberOfFrames(FFrameNumber Length, bool bShouldTransact)
{
	ValidateModel();
	
	const FFrameNumber CurrentNumberOfFrames = Model->GetNumberOfFrames();
	const int32 DeltaFrames = FMath::Abs(Length.Value - CurrentNumberOfFrames.Value);
	
	const FFrameNumber T0 = Length > CurrentNumberOfFrames ? CurrentNumberOfFrames : CurrentNumberOfFrames - DeltaFrames;
	const FFrameNumber T1 = Length > CurrentNumberOfFrames ? Length : CurrentNumberOfFrames;

	ResizeNumberOfFrames(Length, T0, T1, bShouldTransact);	
}

void UAnimSequencerController::ResizeNumberOfFrames(FFrameNumber NewLength, FFrameNumber T0, FFrameNumber T1, bool bShouldTransact)
{
	ValidateModel();

	if (UMovieScene* MovieScene = Model->GetMovieScene())
	{
		const TRange<FFrameNumber> PlayRange = MovieScene->GetPlaybackRange();
		if (NewLength >= 1)
		{
			if (NewLength != Model->GetNumberOfFrames())
			{
				// Ensure that T0 is within the current play range
				if (PlayRange.Contains(T0))
				{
					// Ensure that the start and end length of either removal or insertion are valid
					if (T0 < T1)
					{
						IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
						FTransaction Transaction = ConditionalTransaction(LOCTEXT("ResizePlayLength", "Resizing Play Length"), bShouldTransact);
						
						const FFrameRate CurrentFrameRate = MovieScene->GetDisplayRate();
						const FFrameNumber CurrentNumberOfFrames = Model->GetNumberOfFrames();

						ConditionalAction<UE::Anim::FResizePlayLengthInFramesAction>(bShouldTransact, Model.Get(), T0, T1);
						
						SetMovieSceneRange(NewLength);

						FSequenceLengthChangedPayload Payload;

						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						Payload.T0 = static_cast<float>(CurrentFrameRate.AsSeconds(T0));
						Payload.T1 = static_cast<float>(CurrentFrameRate.AsSeconds(T1));
						Payload.PreviousLength = static_cast<float>(CurrentFrameRate.AsSeconds(CurrentNumberOfFrames));
						PRAGMA_ENABLE_DEPRECATION_WARNINGS

						Payload.PreviousNumberOfFrames = CurrentNumberOfFrames;
						Payload.Frame0 = T0;
						Payload.Frame1 = T1;
						
						Model->GetNotifier().Notify<FSequenceLengthChangedPayload>(EAnimDataModelNotifyType::SequenceLengthChanged, Payload);
					}
					else
					{
						ReportErrorf(LOCTEXT("InvalidEndTimeError", "Invalid T1, smaller that T0 value: T0 {0}, T1 {1}"), FText::AsNumber(T0.Value), FText::AsNumber(T0.Value));
					}
				}
				else
				{
					ReportErrorf(LOCTEXT("InvalidStartTimeError", "Invalid T0, not within existing play range: T0 {0}, Play Length {1}"), FText::AsNumber(T0.Value), FText::AsNumber(Model->GetPlayLength()));
				}
			}
			else if (Model->bPopulated)
			{
				Reportf(ELogVerbosity::Display, LOCTEXT("SamePlayLengthWarning", "New play length is same as existing one: {0} frames"), FText::AsNumber(NewLength.Value));
			}
		}
		else
		{
			ReportErrorf(LOCTEXT("InvalidPlayLengthError", "Invalid play length value provided: {0} frames"), FText::AsNumber(NewLength.Value));
		}
	}	
}

void UAnimSequencerController::ResizeInFrames(FFrameNumber NewLength, FFrameNumber T0, FFrameNumber T1, bool bShouldTransact)
{
	ValidateModel();

	const int32 CurrentNumberOFrames = Model->GetNumberOfFrames();

	if (UMovieScene* MovieScene = Model->GetMovieScene())
	{
		const TRange<FFrameNumber> PlayRange = MovieScene->GetPlaybackRange();
		if (NewLength >= 1)
		{
			if (NewLength != Model->GetNumberOfFrames())
			{
				// Ensure that T0 is within the current play range
				if (PlayRange.Contains(T0))
				{
					// Ensure that the start and end length of either removal or insertion are valid
					if (T0 < T1)
					{
						IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
						FBracket Bracket = ConditionalBracket(LOCTEXT("ResizeModel", "Resizing Animation Data"), bShouldTransact);
						const bool bInserted = NewLength > CurrentNumberOFrames;
						ResizeNumberOfFrames(NewLength, T0, T1, bShouldTransact);

						const FFrameRate CurrentFrameRate = MovieScene->GetDisplayRate();
						ResizeCurves(static_cast<float>(CurrentFrameRate.AsSeconds(NewLength)), bInserted, static_cast<float>(CurrentFrameRate.AsSeconds(T0)), static_cast<float>(CurrentFrameRate.AsSeconds(T1)), bShouldTransact);
						ResizeAttributes(static_cast<float>(CurrentFrameRate.AsSeconds(NewLength)), bInserted, static_cast<float>(CurrentFrameRate.AsSeconds(T0)), static_cast<float>(CurrentFrameRate.AsSeconds(T1)), bShouldTransact);
					}
					else
					{
						ReportErrorf(LOCTEXT("InvalidEndTimeError", "Invalid T1, smaller that T0 value: T0 {0}, T1 {1}"), FText::AsNumber(T0.Value), FText::AsNumber(T1.Value));
					}
				}
				else
				{
					ReportErrorf(LOCTEXT("InvalidStartTimeError", "Invalid T0, not within existing play range: T0 {0}, Play Length {1}"), FText::AsNumber(T0.Value), FText::AsNumber(CurrentNumberOFrames));
				}			
			}
			else if (Model->bPopulated)
			{
				Reportf(ELogVerbosity::Display,LOCTEXT("SameGetPlayLengthWarning", "New play length is same as existing one: {0} frames"), FText::AsNumber(CurrentNumberOFrames));
			}
		}
		else
		{
			ReportErrorf(LOCTEXT("InvalidGetPlayLengthError", "Invalid play length value provided: {0} frames"), FText::AsNumber(CurrentNumberOFrames));
		}
	}
}

void UAnimSequencerController::SetPlayLength(float Length, bool bShouldTransact /*= true*/)
{
	SetNumberOfFrames(ConvertSecondsToFrameNumber(Length), bShouldTransact);
}

void UAnimSequencerController::SetMovieSceneRange(FFrameNumber InFrameNumber) const
{
	ValidateModel();

	UMovieScene* MovieScene = Model->MovieScene;
	const FFrameRate TickRate = MovieScene->GetTickResolution();
	const FFrameTime TickResolutionFrameNumber = FFrameRate::TransformTime(FFrameTime(InFrameNumber), Model->GetFrameRate(), TickRate);
	
	const TRange<FFrameNumber> DataRange = TRange<FFrameNumber>::Inclusive(FFrameNumber(0), TickResolutionFrameNumber.GetFrame());
	MovieScene->SetPlaybackRange(DataRange, false);
	MovieScene->SetPlaybackRangeLocked(false);
	
	FMovieSceneEditorData& EditorData = MovieScene->GetEditorData();
	EditorData.ViewStart = EditorData.WorkStart = 0.0;
	EditorData.ViewEnd = EditorData.WorkEnd = Model->GetFrameRate().AsSeconds(InFrameNumber);
	MovieScene->SetWorkingRange(EditorData.ViewStart, EditorData.ViewEnd);
	MovieScene->SetViewRange(EditorData.ViewStart, EditorData.ViewEnd);

	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		Section->SectionRange = DataRange;
	}
}

void UAnimSequencerController::EnsureModelIsInitialized() const
{
	ensure(Model.Get());
	ensure(Model->MovieScene);
	const UMovieSceneControlRigParameterTrack* Track = Model->MovieScene->FindTrack<UMovieSceneControlRigParameterTrack>();
	if (ensure(Track))
	{
		ensure(Track->GetAllSections().Num() > 0);

		for (const UMovieSceneSection* Section : Track->GetAllSections())
		{
			const UMovieSceneControlRigParameterSection* CRSection = CastChecked<UMovieSceneControlRigParameterSection>(Section);
			ensure(CRSection->GetControlRig());
			ensure(CRSection->GetControlRig()->GetClass() == UFKControlRig::StaticClass());
		}
	}
}

bool UAnimSequencerController::IgnoreSkeletonValidation() const
{
	// Only ever ignore an invalid skeleton if we are populating the model, and the skeleton is null. This happens when the skeleton was moved/deleted, but should not prevent from copying over the bone/curve keys.
	return Model->GetSkeleton() == nullptr && Model->bPopulated == false;
}

bool UAnimSequencerController::ShouldTransact(bool bInLocalShouldTransact) const
{
	return bInLocalShouldTransact && (BracketDepth == 0 || bShouldTransactBracket);
}

void UAnimSequencerController::ResizePlayLength(float Length, float T0, float T1, bool bShouldTransact)
{
	ResizeNumberOfFrames(ConvertSecondsToFrameNumber(Length), ConvertSecondsToFrameNumber(T0), ConvertSecondsToFrameNumber(T1), bShouldTransact);
}

void UAnimSequencerController::Resize(float Length, float T0, float T1, bool bShouldTransact /*= true*/)
{
	ValidateModel();
	
	ResizeInFrames(ConvertSecondsToFrameNumber(Length), ConvertSecondsToFrameNumber(T0), ConvertSecondsToFrameNumber(T1), bShouldTransact);
}

void UAnimSequencerController::SetFrameRate(FFrameRate FrameRate, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	// Disallow invalid frame-rates, or 0.0 intervals
	const float FrameRateInterval = static_cast<float>(FrameRate.AsInterval());
	if (FrameRate.IsValid() && !FMath::IsNearlyZero(FrameRateInterval) && FrameRateInterval > 0.f)
	{
		if (UMovieScene* MovieScene = Model->GetMovieScene())
		{
			// Need to verify frame-rate
			const FFrameRate CurrentFrameRate = MovieScene->GetDisplayRate();

			if (FrameRate.IsMultipleOf(CurrentFrameRate) || FrameRate.IsFactorOf(CurrentFrameRate) || !Model->bPopulated)
			{
				IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetFrameRate", "Setting Frame Rate"), bShouldTransact);
				ConditionalAction<UE::Anim::FSetFrameRateAction>(bShouldTransact, Model.Get());

				const FFrameNumber CurrentNumberOfFrames = Model->GetNumberOfFrames();
		
				MovieScene->SetDisplayRate(FrameRate);
				MovieScene->SetTickResolutionDirectly(FrameRate);

				const FFrameTime TransformedFrameTime = FFrameRate::TransformTime(CurrentNumberOfFrames, CurrentFrameRate, FrameRate);
				if (Model->bPopulated && !FMath::IsNearlyZero(TransformedFrameTime.GetSubFrame(), KINDA_SMALL_NUMBER))
				{ 
					static const FNumberFormattingOptions DurationFormatOptions = FNumberFormattingOptions()
						.SetMinimumFractionalDigits(8)
						.SetMaximumFractionalDigits(8);
					ReportWarningf(LOCTEXT("TransformedFrametimePrecisionWarning", "Insufficient precision while converting between frame rates {0} and {1}: previous number of frames {2}, converted frame-time {3}"), CurrentFrameRate.ToPrettyText(), FrameRate.ToPrettyText(), FText::AsNumber(CurrentNumberOfFrames.Value), FText::AsNumber(TransformedFrameTime.AsDecimal(), &DurationFormatOptions));
				}

				SetMovieSceneRange(TransformedFrameTime.GetFrame());

				FFrameRateChangedPayload Payload;
				Payload.PreviousFrameRate = CurrentFrameRate;
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::FrameRateChanged, Payload);
			}
			else
			{
				ReportErrorf(LOCTEXT("NonCompatibleFrameRateError", "Incompatible frame rate provided: {0} not a multiple or factor of {1}"), FrameRate.ToPrettyText(), CurrentFrameRate.ToPrettyText());
			}
		}
	}	
	else
	{
		ReportErrorf(LOCTEXT("InvalidFrameRateError", "Invalid frame rate provided: {0}"), FrameRate.ToPrettyText());
	}
}

bool UAnimSequencerController::RemoveBoneTracksMissingFromSkeleton(const USkeleton* Skeleton, bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return false;
	}

	if (Skeleton)
	{
		if(UControlRig* ControlRig = Model->GetControlRig())
		{
			TSharedPtr<IControlRigObjectBinding> Binding = ControlRig->GetObjectBinding();
			const UObject* BoundControlRigObject = Binding.IsValid() ? Binding->GetBoundObject() : nullptr;
			if (BoundControlRigObject && BoundControlRigObject == Skeleton)
			{
				TArray<FName> TracksToBeRemoved;
				const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();

				IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
				if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
				{
					const TArray<FTransformParameterNameAndCurves>& BoneTrackCurves = Section->GetTransformParameterNamesAndCurves();
					for (const FTransformParameterNameAndCurves& BoneCurve : BoneTrackCurves)
					{
						const FName ExpectedBoneName = UFKControlRig::GetControlTargetName(BoneCurve.ParameterName, ERigElementType::Bone);
					
						if (ReferenceSkeleton.FindBoneIndex(ExpectedBoneName) == INDEX_NONE)
						{
							// Bone does not exist in the skeleton
							TracksToBeRemoved.Add(ExpectedBoneName);
						}
					}
				}

				if (TracksToBeRemoved.Num())
				{
					FBracket Bracket = ConditionalBracket(LOCTEXT("RemoveBoneTracksMissingFromSkeleton", "Validating Bone Animation Track Data against Skeleton"), bShouldTransact);
					for (const FName& TrackName : TracksToBeRemoved)
					{
						RemoveBoneTrack(TrackName, bShouldTransact);
					}
				}

				return TracksToBeRemoved.Num() > 0;
			}
			else
			{
				if (!BoundControlRigObject)
				{
					ReportErrorf(LOCTEXT("MissingControlRigBinding", "ControlRig has no bound object; cannot validate against Skeleton {0}"),  FText::FromString(Skeleton->GetName()));
				}
				else
				{
					ReportErrorf(LOCTEXT("SkeletonNotBoundToControlRig", "Provided Skeleton {0} does not match bound ControlRigObject {1}"), FText::FromString(Skeleton->GetName()), FText::FromString(BoundControlRigObject->GetName()));
				}
			}
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidSkeletonError", "Invalid USkeleton supplied"));
	}

	return false;
}

void UAnimSequencerController::UpdateAttributesFromSkeleton(const USkeleton* Skeleton, bool bShouldTransact)
{
	ValidateModel();
	
	if (Skeleton)
	{
		// Verifying that bone (names) for attribute data exist on new skeleton
		TArray<FAnimationAttributeIdentifier> ToRemoveIdentifiers;
		TArray<TPair<FAnimationAttributeIdentifier, int32>> ToDuplicateIdentifiers;
			    
		for (const FAnimatedBoneAttribute& Attribute : Model->AnimatedBoneAttributes)
		{
			const int32 NewBoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(Attribute.Identifier.GetBoneName());

			if (NewBoneIndex == INDEX_NONE)
			{
				ToRemoveIdentifiers.Add(Attribute.Identifier);
			}
			else if(NewBoneIndex != Attribute.Identifier.GetBoneIndex())
			{						
				ToDuplicateIdentifiers.Add(TPair<FAnimationAttributeIdentifier, int32>(Attribute.Identifier, NewBoneIndex));
			}
		}

		if (ToRemoveIdentifiers.Num() || ToDuplicateIdentifiers.Num())
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			FBracket Bracket = ConditionalBracket(LOCTEXT("VerifyAttributeBoneNames", "Remapping Animation Attribute Data"), bShouldTransact);
			for (const FAnimationAttributeIdentifier& Identifier : ToRemoveIdentifiers)
			{
				RemoveAttribute(Identifier, bShouldTransact);
			}
			
			for (const TPair<FAnimationAttributeIdentifier, int32>& Pair : ToDuplicateIdentifiers)
			{
				const FAnimationAttributeIdentifier& DuplicateIdentifier = Pair.Key;								
				FAnimationAttributeIdentifier NewIdentifier(DuplicateIdentifier.GetName(), Pair.Value, DuplicateIdentifier.GetBoneName(), DuplicateIdentifier.GetType());

				DuplicateAttribute(Pair.Key, NewIdentifier, bShouldTransact);
				RemoveAttribute(Pair.Key, bShouldTransact);
			}	
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidSkeletonError", "Invalid USkeleton supplied"));
	}
}

void UAnimSequencerController::ResetModel(bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (Model->bPopulated)
	{
		IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
		FBracket Bracket = ConditionalBracket(LOCTEXT("ResetModel", "Clearing Animation Data"), bShouldTransact);

		RemoveAllBoneTracks(bShouldTransact);

		RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Float, bShouldTransact);
		RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Transform, bShouldTransact);
		RemoveAllAttributes(bShouldTransact);

		Model->bPopulated = false;
		SetFrameRate(FCommonFrameRates::FPS_30(), bShouldTransact);
		SetNumberOfFrames(1, bShouldTransact);
		Model->bPopulated = true;

		Model->GetNotifier().Notify(EAnimDataModelNotifyType::Reset);
	}
}

bool UAnimSequencerController::AddCurve(const FAnimationCurveIdentifier& CurveId, int32 CurveFlags /*= EAnimAssetCurveFlags::AACF_Editable*/, bool bShouldTransact /*= true*/)
{
	ValidateModel();
	if (CurveId.IsValid())
	{
		if (IsSupportedCurveType(CurveId.CurveType))
		{
			if (!Model->FindCurve(CurveId))
			{
				IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("AddRawCurve", "Adding Animation Curve"), bShouldTransact);

				FCurveAddedPayload Payload;
				Payload.Identifier = CurveId;
				
				auto AddNewCurve = [this, CurveId, CurveName = CurveId.CurveName, CurveFlags](auto& CurveTypeArray)
				{
					CurveTypeArray.Add({ CurveName, CurveFlags});					
					Model->CurveIdentifierToMetaData.FindOrAdd(CurveId).Flags = CurveFlags;
				};

				switch (CurveId.CurveType)
				{
				case ERawCurveTrackTypes::RCT_Transform:
					AddNewCurve(Model->LegacyCurveData.TransformCurves);
					break;
				case ERawCurveTrackTypes::RCT_Float:
					AddNewCurve(Model->LegacyCurveData.FloatCurves);
					if(!AddCurveControl(CurveId.CurveName))
					{
						ReportError(LOCTEXT("FailedtoAddCurveControl", "Failed to add curve control"));
					}
					Model->CurveIdentifierToMetaData.FindChecked(CurveId).Color = FAnimCurveBase::MakeColor(CurveId.CurveName);
					break;
				default:
					{
						checkf(false, TEXT("Unsupported curve type"));
						break;
					}
				}

				ConditionalAction<UE::Anim::FRemoveCurveAction>(bShouldTransact,  CurveId);
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveAdded, Payload);

				return true;
			}
			else
			{
				const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
				ReportWarningf(LOCTEXT("ExistingCurveNameWarning", "Curve with name {0} and type {1} ({2}) already exists"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(CurveId.CurveType)));
			}			
		}
		else 
		{
			const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
			ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(CurveId.CurveType)));
		}		
	}
	else
	{
		const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
		ReportWarningf(LOCTEXT("InvalidCurveIdentifierWarning", "Invalid curve identifier provided: name: {0}, type: {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
	}

	return false;
}

bool UAnimSequencerController::DuplicateCurve(const FAnimationCurveIdentifier& CopyCurveId, const FAnimationCurveIdentifier& NewCurveId, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	ERawCurveTrackTypes SupportedCurveType = CopyCurveId.CurveType;

	if (CopyCurveId.IsValid() && NewCurveId.IsValid())
	{
		if (IsSupportedCurveType(SupportedCurveType))
		{
			if (CopyCurveId.CurveType == NewCurveId.CurveType)
			{
				if (Model->FindCurve(CopyCurveId))
				{
					if (!Model->FindCurve(NewCurveId))
					{
						IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
						FTransaction Transaction = ConditionalTransaction(LOCTEXT("CopyRawCurve", "Duplicating Animation Curve"), bShouldTransact);

						auto DuplicateCurve = [NewCurveId, NewCurveName = NewCurveId.CurveName, this](auto& LegacyCurveDataArray, const auto& SourceCurve)
						{
							auto& DuplicatedCurve = LegacyCurveDataArray.Add_GetRef( { NewCurveName, SourceCurve.GetCurveTypeFlags() });
							DuplicatedCurve.CopyCurve(SourceCurve);
							Model->CurveIdentifierToMetaData.FindOrAdd(NewCurveId).Flags = SourceCurve.GetCurveTypeFlags();
							Model->CurveIdentifierToMetaData.FindOrAdd(NewCurveId).Color = SourceCurve.GetColor();
						};
						
						switch (SupportedCurveType)
						{
						case ERawCurveTrackTypes::RCT_Transform:
							DuplicateCurve(Model->LegacyCurveData.TransformCurves, Model->GetTransformCurve(CopyCurveId));
							break;
						case ERawCurveTrackTypes::RCT_Float:
							{
								DuplicateCurve(Model->LegacyCurveData.FloatCurves, Model->GetFloatCurve(CopyCurveId));
															
								if(!DuplicateCurveControl(CopyCurveId.CurveName, NewCurveId.CurveName))
								{
									ReportError(LOCTEXT("FailedToDuplicateCurveControl", "Failed to duplicate curve control"));
								}
								break;
							}
						default:
							{
								checkf(false, TEXT("Unsupported curve type"));
								break;
							}
						}

						FCurveAddedPayload Payload;
						Payload.Identifier = NewCurveId;
						Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveAdded, Payload);

						ConditionalAction<UE::Anim::FRemoveCurveAction>(bShouldTransact,  NewCurveId);

						return true;
					}
					else
					{
						const FString CurveTypeAsString = GetCurveTypeValueName(NewCurveId.CurveType);
						ReportWarningf(LOCTEXT("ExistingCurveNameWarning", "Curve with name {0} and type {1} ({2}) already exists"), FText::FromName(NewCurveId.CurveName), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(NewCurveId.CurveType)));
					}
				}
				else
				{
					const FString CurveTypeAsString = GetCurveTypeValueName(CopyCurveId.CurveType);
					ReportWarningf(LOCTEXT("CurveNameToDuplicateNotFoundWarning", "Could not find curve with name {0} and type {1} ({2}) for duplication"), FText::FromName(NewCurveId.CurveName), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(NewCurveId.CurveType)));
				}
			}
		}
		else
		{
			const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
			ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(SupportedCurveType)));
		}
	}
	else
	{
		ReportWarningf(LOCTEXT("DuplicateInvalidCurveIdentifierWarning", "Invalid curve identifier provided for duplicate: copy: {0}, new: {1}"), FText::FromName(CopyCurveId.CurveName), FText::FromName(NewCurveId.CurveName));
	}

	return false;
}

bool UAnimSequencerController::RemoveCurve(const FAnimationCurveIdentifier& CurveId, bool bShouldTransact /*= true*/)
{
	ValidateModel();
	const ERawCurveTrackTypes SupportedCurveType = CurveId.CurveType;

	if (CurveId.IsValid())
	{
		if (IsSupportedCurveType(CurveId.CurveType))
		{
			if (Model->FindCurve(CurveId) != nullptr)
			{
				IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("RemoveCurve", "Removing Animation Curve"), bShouldTransact);

				switch (SupportedCurveType)
				{
				case ERawCurveTrackTypes::RCT_Transform:
					{
						const FTransformCurve& TransformCurve = Model->GetTransformCurve(CurveId);
						ConditionalAction<UE::Anim::FAddTransformCurveAction>(bShouldTransact,  CurveId, TransformCurve.GetCurveTypeFlags(), TransformCurve);
						Model->LegacyCurveData.TransformCurves.RemoveAll([Name = TransformCurve.GetName()](const FTransformCurve& ToRemoveCurve) { return ToRemoveCurve.GetName() == Name; });
						Model->CurveIdentifierToMetaData.Remove(CurveId);
						break;
					}
				case ERawCurveTrackTypes::RCT_Float:
					{
						const FFloatCurve& FloatCurve = Model->GetFloatCurve(CurveId);
						ConditionalAction<UE::Anim::FAddFloatCurveAction>(bShouldTransact,  CurveId, FloatCurve.GetCurveTypeFlags(), FloatCurve.FloatCurve.GetConstRefOfKeys(), FloatCurve.Color);
						Model->LegacyCurveData.FloatCurves.RemoveAll([Name = FloatCurve.GetName()](const FFloatCurve& ToRemoveCurve) { return ToRemoveCurve.GetName() == Name; });
						Model->CurveIdentifierToMetaData.Remove(CurveId);

						if(!RemoveCurveControl(CurveId.CurveName))
						{
							ReportError(LOCTEXT("FailedtoRemoveCurveControl", "Failed to remove curve control"));
						}
						break;
					}
				default:
					{
						checkf(false, TEXT("Unsupported curve type"));
						break;
					}
				}

				FCurveRemovedPayload Payload;
				Payload.Identifier = CurveId;
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveRemoved, Payload);

				return true;
			}
			else
			{
				const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
				ReportWarningf(LOCTEXT("UnableToFindCurveForRemovalWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
			}
		}
		else
		{
			const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
			ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(CurveId.CurveType)));
		}
	}
	else
	{
		const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
		ReportWarningf(LOCTEXT("InvalidCurveIdentifierWarning", "Invalid curve identifier provided: name: {0}, type: {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
	}
	return false;
}

void UAnimSequencerController::RemoveAllCurvesOfType(ERawCurveTrackTypes SupportedCurveType /*= ERawCurveTrackTypes::RCT_Float*/, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	FBracket Bracket = ConditionalBracket(LOCTEXT("DeleteAllRawCurve", "Deleting All Animation Curve"), bShouldTransact);
	switch (SupportedCurveType)
	{
	case ERawCurveTrackTypes::RCT_Transform:
	{
		TArray<FTransformCurve> TransformCurves = Model->LegacyCurveData.TransformCurves;
		for (const FTransformCurve& Curve : TransformCurves)
		{
			const FAnimationCurveIdentifier CurveId(Curve.GetName(), ERawCurveTrackTypes::RCT_Transform);
			RemoveCurve(CurveId, bShouldTransact);
			Model->CurveIdentifierToMetaData.Remove(CurveId);
		}
		break;
	}
	case ERawCurveTrackTypes::RCT_Float:
	{
		TArray<FFloatCurve> FloatCurves = Model->LegacyCurveData.FloatCurves;
		for (const FFloatCurve& Curve : FloatCurves)
		{
			const FAnimationCurveIdentifier CurveId(Curve.GetName(), ERawCurveTrackTypes::RCT_Float);
			RemoveCurve(CurveId, bShouldTransact);
			Model->CurveIdentifierToMetaData.Remove(CurveId);
		}
		break;
	}
	case ERawCurveTrackTypes::RCT_Vector:
	default:
		const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
		ReportWarningf(LOCTEXT("InvalidCurveTypeWarning", "Invalid curve type provided: {0} ({1})"), FText::FromString(CurveTypeAsString), FText::AsNumber(static_cast<int32>(SupportedCurveType)));
	}
}

bool UAnimSequencerController::SetCurveFlag(const FAnimationCurveIdentifier& CurveId, EAnimAssetCurveFlags Flag, bool bState /*= true*/, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	const ERawCurveTrackTypes SupportedCurveType = CurveId.CurveType;

	FAnimCurveBase* Curve = nullptr;

	if (SupportedCurveType == ERawCurveTrackTypes::RCT_Float)
	{
		Curve = Model->FindMutableFloatCurveById(CurveId);
	}
	else if (SupportedCurveType == ERawCurveTrackTypes::RCT_Transform)
	{
		Curve = Model->FindMutableTransformCurveById(CurveId);
	}
	
	if (Curve)
	{
		IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
		FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetCurveFlag", "Setting Raw Curve Flag"), bShouldTransact);

		const int32 CurrentFlags = Curve->GetCurveTypeFlags();

		ConditionalAction<UE::Anim::FSetCurveFlagsAction>(bShouldTransact,  CurveId, CurrentFlags, SupportedCurveType);

		FCurveFlagsChangedPayload Payload;
		Payload.Identifier = CurveId;
		Payload.OldFlags = Curve->GetCurveTypeFlags();

		Curve->SetCurveTypeFlag(Flag, bState);

		Model->CurveIdentifierToMetaData.FindChecked(CurveId).Flags = Curve->GetCurveTypeFlags();

		Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveFlagsChanged, Payload);

		return true;
	}
	else
	{
		const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
		ReportWarningf(LOCTEXT("UnableToFindCurveWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
	}

	return false;
}

bool UAnimSequencerController::SetCurveFlags(const FAnimationCurveIdentifier& CurveId, int32 Flags, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	FAnimCurveBase* Curve = nullptr;

	const ERawCurveTrackTypes SupportedCurveType = CurveId.CurveType;

	if (SupportedCurveType == ERawCurveTrackTypes::RCT_Float)
	{
		Curve = Model->FindMutableFloatCurveById(CurveId);
	}
	else if (SupportedCurveType == ERawCurveTrackTypes::RCT_Transform)
	{
		Curve = Model->FindMutableTransformCurveById(CurveId);
	}

	if (Curve)
	{
		IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
		FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetCurveFlags", "Setting Raw Curve Flags"), bShouldTransact);

		const int32 CurrentFlags = Curve->GetCurveTypeFlags();

		ConditionalAction<UE::Anim::FSetCurveFlagsAction>(bShouldTransact,  CurveId, CurrentFlags, SupportedCurveType);

		FCurveFlagsChangedPayload Payload;
		Payload.Identifier = CurveId;
		Payload.OldFlags = Curve->GetCurveTypeFlags();

		Curve->SetCurveTypeFlags(Flags);
		Model->CurveIdentifierToMetaData.FindChecked(CurveId).Flags = Curve->GetCurveTypeFlags();

		Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveFlagsChanged, Payload);

		return true;
	}
	else
	{
		const FString CurveTypeAsString = GetCurveTypeValueName(SupportedCurveType);
		ReportWarningf(LOCTEXT("UnableToFindCurveForRemovalWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
	}

	return false;
}

bool UAnimSequencerController::SetTransformCurveKeys(const FAnimationCurveIdentifier& CurveId, const TArray<FTransform>& TransformValues, const TArray<float>& TimeKeys, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (TransformValues.Num() == TimeKeys.Num())
	{
		if (Model->FindMutableTransformCurveById(CurveId) != nullptr)
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			FBracket Bracket = ConditionalBracket(LOCTEXT("SetTransformCurveKeys_Bracket", "Setting Transform Curve Keys"), bShouldTransact);
			
			struct FKeys
			{
				FKeys(int32 NumKeys)
				{
					for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
					{
						ChannelKeys[ChannelIndex].SetNum(NumKeys);
					}
				}

				TArray<FRichCurveKey> ChannelKeys[3];
			};

			FKeys TranslationKeys(TransformValues.Num());
			FKeys RotationKeys(TransformValues.Num());
			FKeys ScaleKeys(TransformValues.Num());

			FKeys* SubCurveKeys[3] = { &TranslationKeys, &RotationKeys, &ScaleKeys };

			// Generate the curve keys
			for (int32 KeyIndex = 0; KeyIndex < TransformValues.Num(); ++KeyIndex)
			{
				const FTransform& Value = TransformValues[KeyIndex];
				const float& Time = TimeKeys[KeyIndex];

				const FVector Translation = Value.GetLocation();
				const FVector Rotation = Value.GetRotation().Euler();
				const FVector Scale = Value.GetScale3D();

				auto SetKey = [Time, KeyIndex](FKeys& Key, const FVector& Vector)
				{
					for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
					{
						Key.ChannelKeys[ChannelIndex][KeyIndex] = FRichCurveKey(Time, static_cast<float>(Vector[ChannelIndex]));
					}
				};

				SetKey(TranslationKeys, Translation);
				SetKey(RotationKeys, Rotation);
				SetKey(ScaleKeys, Scale);
			}
			
			for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
			{
				const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
				const FKeys* CurveKeys = SubCurveKeys[SubCurveIndex];
				for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
				{
					const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
					FAnimationCurveIdentifier TargetCurveIdentifier = CurveId;
					UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
					SetCurveKeys(TargetCurveIdentifier, CurveKeys->ChannelKeys[ChannelIndex], bShouldTransact);
				}
			}

			return true;
		}
		else
		{
			ReportWarningf(LOCTEXT("UnableToFindTransformCurveWarning", "Unable to find transform curve: {0}"), FText::FromName(CurveId.CurveName));
		}
	}
	else
	{
		// time value mismatch
		ReportWarningf(LOCTEXT("InvalidNumberOfTimeAndKeyEntriesWarning", "Number of times and key entries do not match: number of time values {0}, number of key values {1}"), FText::AsNumber(TimeKeys.Num()), FText::AsNumber(TransformValues.Num()));
	}

	return false;	
}


bool UAnimSequencerController::SetTransformCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, const FTransform& Value, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (Model->FindMutableTransformCurveById(CurveId) != nullptr)
	{
		IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
		FBracket Bracket = ConditionalBracket(LOCTEXT("AddTransformCurveKey_Bracket", "Setting Transform Curve Key"), bShouldTransact);
		struct FKeys
		{
			FRichCurveKey ChannelKeys[3];
		};

		FKeys VectorKeys[3];
		
		// Generate the rich curve keys		
		const FVector Translation = Value.GetLocation();
		const FVector Rotation = Value.GetRotation().Euler();
		const FVector Scale = Value.GetScale3D();

		auto SetKey = [Time](FKeys& Key, const FVector& Vector)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
			{
				Key.ChannelKeys[ChannelIndex] = FRichCurveKey(Time, static_cast<float>(Vector[ChannelIndex]));
			}
		};

		SetKey(VectorKeys[0], Translation);
		SetKey(VectorKeys[1], Rotation);
		SetKey(VectorKeys[2], Scale);
		
		for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
		{
			const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
			const FKeys& VectorCurveKeys = VectorKeys[SubCurveIndex];
			for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
			{
				const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
				FAnimationCurveIdentifier TargetCurveIdentifier = CurveId;
				UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
				SetCurveKey(TargetCurveIdentifier, VectorCurveKeys.ChannelKeys[ChannelIndex], bShouldTransact);
			}
		}

		return true;
	}
	else
	{
		ReportWarningf(LOCTEXT("UnableToFindTransformCurveWarning", "Unable to find transform curve: {0}"), FText::FromName(CurveId.CurveName));
	}

	return false;
}

bool UAnimSequencerController::RemoveTransformCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (Model->FindMutableTransformCurveById(CurveId) != nullptr)
	{
		const FString BaseCurveName = CurveId.CurveName.ToString();
		const TArray<FString> SubCurveNames = { TEXT( "Translation"), TEXT( "Rotation"), TEXT( "Scale") };
		const TArray<FString> ChannelCurveNames = { TEXT("X"), TEXT("Y"), TEXT("Z") };

		IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
		FBracket Bracket = ConditionalBracket(LOCTEXT("RemoveTransformCurveKey_Bracket", "Deleting Animation Transform Curve Key"), bShouldTransact);
		
		for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
		{
			const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
			for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
			{
				const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
				FAnimationCurveIdentifier TargetCurveIdentifier = CurveId;
				UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
				RemoveCurveKey(TargetCurveIdentifier, Time, bShouldTransact);
			}
		}

		return true;
	}
	else
	{
		ReportWarningf(LOCTEXT("UnableToFindTransformCurveWarning", "Unable to find transform curve: {0}"), FText::FromName(CurveId.CurveName));
	}

	return false;
}

bool UAnimSequencerController::RenameCurve(const FAnimationCurveIdentifier& CurveToRenameId, const FAnimationCurveIdentifier& NewCurveId, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	if (NewCurveId.IsValid())
	{
		if (CurveToRenameId != NewCurveId)
		{
			if (CurveToRenameId.CurveType == NewCurveId.CurveType)
			{
				if (FAnimCurveBase* Curve = Model->FindMutableCurveById(CurveToRenameId))
				{
					IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
					FTransaction Transaction = ConditionalTransaction(LOCTEXT("RenameCurve", "Renaming Curve"), bShouldTransact);

					FCurveRenamedPayload Payload;
					Payload.Identifier = FAnimationCurveIdentifier(Curve->GetName(), CurveToRenameId.CurveType);

					Curve->SetName(NewCurveId.CurveName);
					Payload.NewIdentifier = NewCurveId;

					const FAnimationCurveMetaData CurveMetaData = Model->CurveIdentifierToMetaData.FindChecked(CurveToRenameId);
					Model->CurveIdentifierToMetaData.Remove(CurveToRenameId);
					Model->CurveIdentifierToMetaData.Add(NewCurveId, CurveMetaData);

					const bool bControlNeedsRenaming = CurveToRenameId.CurveName != NewCurveId.CurveName;

					if(CurveToRenameId.CurveType == ERawCurveTrackTypes::RCT_Float && (bControlNeedsRenaming && !RenameCurveControl(CurveToRenameId.CurveName, NewCurveId.CurveName)))
					{
						ReportErrorf(LOCTEXT("FailedToRenameCurveControl", "Failed to rename RigCurve with name {0}"), FText::FromName(CurveToRenameId.CurveName));
					}

					ConditionalAction<UE::Anim::FRenameCurveAction>(bShouldTransact,  NewCurveId, CurveToRenameId);

					Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveRenamed, Payload);

					return true;
				}
				else
				{
					const FString CurveTypeAsString = GetCurveTypeValueName(CurveToRenameId.CurveType);
					ReportWarningf(LOCTEXT("UnableToFindCurveWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveToRenameId.CurveName), FText::FromString(CurveTypeAsString));
				}
			}
			else
			{
				const FString CurrentCurveTypeAsString = GetCurveTypeValueName(CurveToRenameId.CurveType);
				const FString NewCurveTypeAsString = GetCurveTypeValueName(NewCurveId.CurveType);
				ReportWarningf(LOCTEXT("MismatchOfCurveTypesWarning", "Different curve types provided between current and new curve names: {0} ({1}) and {2} ({3})"), FText::FromName(CurveToRenameId.CurveName), FText::FromString(CurrentCurveTypeAsString),
					FText::FromName(NewCurveId.CurveName), FText::FromString(NewCurveTypeAsString));
			}
		}
		else
		{
			ReportWarningf(LOCTEXT("MatchingCurveNamesWarning", "Provided curve names are the same: {0}"), FText::FromName(CurveToRenameId.CurveName));
		}		
	}
	else
	{
		ReportWarningf(LOCTEXT("InvalidCurveIdentiferProvidedWarning", "Invalid new curve identifier provided: {0}"), FText::FromName(NewCurveId.CurveName));
	}

	return false;
}

bool UAnimSequencerController::SetCurveColor(const FAnimationCurveIdentifier& CurveId, FLinearColor Color, bool bShouldTransact)
{
	ValidateModel();

	if (CurveId.IsValid())
	{
		if (CurveId.CurveType == ERawCurveTrackTypes::RCT_Float)
		{
			if (const FFloatCurve* Curve = Model->FindMutableFloatCurveById(CurveId))
			{
				IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("ChangingCurveColor", "Changing Curve Color"), bShouldTransact);

				ConditionalAction<UE::Anim::FSetCurveColorAction>(bShouldTransact,  CurveId, Curve->Color);

				Model->CurveIdentifierToMetaData.FindChecked(CurveId).Color = Color;

				FCurveChangedPayload Payload;
				Payload.Identifier = CurveId;
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveColorChanged, Payload);

				return true;				
			}
			else
			{
				const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
				ReportWarningf(LOCTEXT("UnableToFindCurveWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
			}
		}
		else
		{
			ReportWarning(LOCTEXT("NonSupportedCurveColorSetWarning", "Changing curve color is currently only supported for float curves"));
		}
	}
	else
	{
		ReportWarningf(LOCTEXT("InvalidCurveIdentifier", "Invalid Curve Identifier : {0}"), FText::FromName(CurveId.CurveName));
	}	

	return false;
}

bool UAnimSequencerController::SetCurveComment(const FAnimationCurveIdentifier& CurveId, const FString& Comment, bool bShouldTransact)
{
	ValidateModel();

	if (CurveId.IsValid())
	{
		if (CurveId.CurveType == ERawCurveTrackTypes::RCT_Float)
		{
			if (const FFloatCurve* Curve = Model->FindMutableFloatCurveById(CurveId))
			{
				IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("ChangingCurveComment", "Changing Curve Comment"), bShouldTransact);

				ConditionalAction<UE::Anim::FSetCurveCommentAction>(bShouldTransact,  CurveId, Curve->Comment);

				Model->CurveIdentifierToMetaData.FindChecked(CurveId).Comment = Comment;

				FCurveChangedPayload Payload;
				Payload.Identifier = CurveId;
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveCommentChanged, Payload);

				return true;
			}
			else
			{
				const FString CurveTypeAsString = GetCurveTypeValueName(CurveId.CurveType);
				ReportWarningf(LOCTEXT("UnableToFindCurveWarning", "Unable to find curve: {0} of type {1}"), FText::FromName(CurveId.CurveName), FText::FromString(CurveTypeAsString));
			}
		}
		else
		{
			ReportWarning(LOCTEXT("NonSupportedCurveCommentSetWarning", "Changing curve comment is currently only supported for float curves"));
		}
	}
	else
	{
		ReportWarningf(LOCTEXT("InvalidCurveIdentifier", "Invalid Curve Identifier : {0}"), FText::FromName(CurveId.CurveName));
	}	

	return false;
}


bool UAnimSequencerController::ScaleCurve(const FAnimationCurveIdentifier& CurveId, float Origin, float Factor, bool bShouldTransact /*= true*/)
{
	ValidateModel();

	const ERawCurveTrackTypes SupportedCurveType = CurveId.CurveType;
	if (SupportedCurveType == ERawCurveTrackTypes::RCT_Float)
	{
		if (FFloatCurve* Curve = Model->FindMutableFloatCurveById(CurveId))
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("ScalingCurve", "Scaling Curve"), bShouldTransact);

			Curve->FloatCurve.ScaleCurve(Origin, Factor);

			FCurveScaledPayload Payload;
			Payload.Identifier = CurveId;
			Payload.Factor = Factor;
			Payload.Origin = Origin;
			
			ConditionalAction<UE::Anim::FScaleCurveAction>(bShouldTransact,  CurveId, Origin, 1.0f / Factor, SupportedCurveType);

			if (!SetCurveControlKeys(CurveId.CurveName, Curve->FloatCurve.GetConstRefOfKeys()))
			{
				ReportError(LOCTEXT("FailedToScaleControlCurve", "Failed to scale curve control"));
			}

			Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveScaled, Payload);

			return true;
		}
		else
		{
			ReportWarningf(LOCTEXT("UnableToFindFloatCurveWarning", "Unable to find float curve: {0}"), FText::FromName(CurveId.CurveName));
		}
	}
	else
	{
		ReportWarning(LOCTEXT("NonSupportedCurveScalingWarning", "Scaling curves is currently only supported for float curves"));
	}

	return false;
}

bool UAnimSequencerController::SetCurveKey(const FAnimationCurveIdentifier& CurveId, const FRichCurveKey& Key, bool bShouldTransact)
{
	ValidateModel();

	if (FRichCurve* RichCurve = Model->GetMutableRichCurve(CurveId))
	{
		FCurveChangedPayload Payload;
		Payload.Identifier = CurveId;

		auto SetControlCurve = [this, CurveId, Key](bool bUpdate, float& OutKeyTiming)
		{
			// FMovieSceneFloatChannel stores keys along frames rather than time-based keying, this means Key.Time can be different than actual key set on the float-channel (max 240k fps). So adjust the timing using OutKeyTiming to match for undo/redo and rich key mirroring.
			if(CurveId.CurveType == ERawCurveTrackTypes::RCT_Float && !SetCurveControlKey(CurveId.CurveName, Key, bUpdate, OutKeyTiming))
			{
				ReportError(LOCTEXT("FailedToSetCurveControlKey", "Failed to set curve control key"));
				return false;
			}

			return true;
		};

		// Set or add rich curve value
		const FKeyHandle Handle = RichCurve->FindKey(Key.Time);
		if (Handle != FKeyHandle::Invalid())
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetNamedCurveKey", "Setting Curve Key"), bShouldTransact);
			// Cache old value for action
			const FRichCurveKey CurrentKey = RichCurve->GetKey(Handle);

			float MovieSceneKeyTiming = Key.Time;
			if (SetControlCurve(true, MovieSceneKeyTiming))
			{
				ConditionalAction<UE::Anim::FSetRichCurveKeyAction>(bShouldTransact,  CurveId, CurrentKey);
				// Set the new value
				RichCurve->SetKeyValue(Handle, Key.Value);
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);
				
				return true;
			}
		}
		else
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("AddNamedCurveKey", "Adding Curve Key"), bShouldTransact);

			float MovieSceneKeyTiming = Key.Time;
			if (SetControlCurve(false, MovieSceneKeyTiming))
			{
				ConditionalAction<UE::Anim::FRemoveRichCurveKeyAction>(bShouldTransact,  CurveId, MovieSceneKeyTiming);
				// Add the new key
				RichCurve->AddKey(MovieSceneKeyTiming, Key.Value);

				Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);
				
				return true;
			}
		}
	}
	else
	{
		ReportErrorf(LOCTEXT("RichCurveNotFoundError", "Unable to find rich curve: curve name {0}"), FText::FromName(CurveId.CurveName));
	}

	return false;
}

bool UAnimSequencerController::RemoveCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, bool bShouldTransact)
{
	ValidateModel();

	if (FRichCurve* RichCurve = Model->GetMutableRichCurve(CurveId))
	{
		FCurveChangedPayload Payload;
		Payload.Identifier = CurveId;

		// Remove key at time value		
		const FKeyHandle Handle = RichCurve->FindKey(Time);
		if (Handle != FKeyHandle::Invalid())
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("RemoveNamedCurveKey", "Removing Curve Key"), bShouldTransact);

			// Cached current value for action
			const FRichCurveKey CurrentKey = RichCurve->GetKey(Handle);
			ConditionalAction<UE::Anim::FAddRichCurveKeyAction>(bShouldTransact,  CurveId, CurrentKey);

			RichCurve->DeleteKey(Handle);
			
			if(CurveId.CurveType == ERawCurveTrackTypes::RCT_Float && !RemoveCurveControlKey(CurveId.CurveName, Time))
			{
				ReportError(LOCTEXT("FailedtoRemoveCurveControlKeys", "Failed to remove curve control key"));
			}

			Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);

			return true;
		}
		else
		{
			ReportErrorf(LOCTEXT("RichCurveKeyNotFoundError", "Unable to find rich curve key: curve name {0}, time {1}"), FText::FromName(CurveId.CurveName), FText::AsNumber(Time));
		}
	}
	else
	{
		ReportErrorf(LOCTEXT("RichCurveNotFoundError", "Unable to find rich curve: curve name {0}"), FText::FromName(CurveId.CurveName));
	}

	return false;
}

bool UAnimSequencerController::SetCurveKeys(const FAnimationCurveIdentifier& CurveId, const TArray<FRichCurveKey>& CurveKeys, bool bShouldTransact)
{
	ValidateModel();

	//if (Model->CurveIdentifierToMetaData.Find(CurveId))
	{
		bool bKeysSet = false;
		if (FRichCurve* RichCurve = Model->GetMutableRichCurve(CurveId))
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("SettingNamedCurveKeys", "Setting Curve Keys"), bShouldTransact);
			ConditionalAction<UE::Anim::FSetRichCurveKeysAction>(bShouldTransact,  CurveId, RichCurve->GetConstRefOfKeys());

			// Set rich curve values
			RichCurve->SetKeys(CurveKeys);

			bKeysSet = true;
		}
		else
		{
			ReportErrorf(LOCTEXT("RichCurveNotFoundError", "Unable to find rich curve: curve name {0}"), FText::FromName(CurveId.CurveName));
		}

		if(CurveId.CurveType == ERawCurveTrackTypes::RCT_Float)
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			if (SetCurveControlKeys(CurveId.CurveName, CurveKeys))
			{
				bKeysSet = true;
			}
			else
			{		
				ReportError(LOCTEXT("FailedtoSetCurveControlKeys", "Failed to set curve control keys"));
			}
		}

		if(bKeysSet)
		{
			FCurveChangedPayload Payload;
			Payload.Identifier = CurveId;
			Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);
		
			return true;
		}
	}
	// else
	// {
	// 	ReportErrorf(LOCTEXT("CurveNotFoundError", "Unable to find curve: curve name {0}"), FText::FromName(CurveId.CurveName));
	// }	

	return false;
}

bool UAnimSequencerController::SetCurveAttributes(const FAnimationCurveIdentifier& CurveId, const FCurveAttributes& Attributes, bool bShouldTransact)
{
	ValidateModel();

	const FRichCurve* RichCurve = Model->GetMutableRichCurve(CurveId);
	if (RichCurve)
	{
		IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
		FTransaction Transaction = ConditionalTransaction(LOCTEXT("SettingNamedCurveAttributes", "Setting Curve Attributes"), bShouldTransact);

		FCurveAttributes CurrentAttributes;
		CurrentAttributes.SetPreExtrapolation(RichCurve->PreInfinityExtrap);
		CurrentAttributes.SetPostExtrapolation(RichCurve->PostInfinityExtrap);

		if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
		{
			UControlRig* ControlRig = Section->GetControlRig();

			if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
			{
				const URigHierarchy* Hierarchy = Model->GetControlRigHierarchy();
				if (Hierarchy ||  IgnoreSkeletonValidation())
				{
					const FRigElementKey CurveKey(URigHierarchy::GetSanitizedName(CurveId.CurveName), ERigElementType::Curve);
					const FRigElementKey CurveControlKey(UFKControlRig::GetControlName(CurveKey.Name, ERigElementType::Curve), ERigElementType::Control);

					const bool bContainsCurve = Hierarchy ? Hierarchy->Contains(CurveKey) : IgnoreSkeletonValidation();
					const bool bContainsCurveControl = Hierarchy ? Hierarchy->Contains(CurveControlKey) : IgnoreSkeletonValidation();

					if(bContainsCurve && bContainsCurveControl)
					{
						const bool bHasCurveControlChannel = Section->HasScalarParameter(CurveControlKey.Name);
						if (!bHasCurveControlChannel)
						{
							Section->AddScalarParameter(CurveControlKey.Name, TOptional<float>(), Model->bPopulated);
						}
										
						FScalarParameterNameAndCurve* ParameterCurvePair = Section->GetScalarParameterNamesAndCurves().FindByPredicate([CurveControlKey](const FScalarParameterNameAndCurve& Parameter)
						{
							return Parameter.ParameterName == CurveControlKey.Name;
						});
						check(ParameterCurvePair);

						if(!bHasCurveControlChannel)
						{
							ParameterCurvePair->ParameterCurve.SetTickResolution(Model->GetFrameRate());
						}
						
						ConditionalAction<UE::Anim::FSetRichCurveAttributesAction>(bShouldTransact, CurveId, CurrentAttributes);	
						if (Attributes.HasPreExtrapolation())
						{							
							ParameterCurvePair->ParameterCurve.PreInfinityExtrap = Attributes.GetPreExtrapolation();
						}

						if (Attributes.HasPostExtrapolation())
						{
							ParameterCurvePair->ParameterCurve.PostInfinityExtrap = Attributes.GetPostExtrapolation();							
						}
					
						FCurveChangedPayload Payload;
						Payload.Identifier = CurveId;
						Model->GetNotifier().Notify(EAnimDataModelNotifyType::CurveChanged, Payload);

						return true;						
					}
				}
			}
		}
	}

	return false;
}

void UAnimSequencerController::NotifyPopulated()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NotifyPopulated);
	ValidateModel();

	Model->bPopulated = true;	
	Model->GetNotifier().Notify(EAnimDataModelNotifyType::Populated);
}

void UAnimSequencerController::NotifyBracketOpen()
{
	ValidateModel();
	Model->GetNotifier().Notify(EAnimDataModelNotifyType::BracketOpened);
}

void UAnimSequencerController::NotifyBracketClosed()
{
	ValidateModel();
	Model->GetNotifier().Notify(EAnimDataModelNotifyType::BracketClosed);
}

bool UAnimSequencerController::AddBoneCurve(FName BoneName, bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return false;
	}

	FTransaction Transaction = ConditionalTransaction(LOCTEXT("AddBoneTrack", "Adding Animation Data Track"), bShouldTransact);
	return InsertBoneTrack(BoneName, INDEX_NONE, bShouldTransact) != INDEX_NONE;
}

int32 UAnimSequencerController::InsertBoneTrack(FName BoneName, int32 DesiredIndex, bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return INDEX_NONE;
	}

	if (!Model->IsValidBoneTrackName(BoneName))
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_InsertBoneTrack);
		if (Model->GetNumBoneTracks() >= MAX_ANIMATION_TRACKS)
		{
			ReportWarningf(LOCTEXT("MaxNumberOfTracksReachedWarning", "Cannot add track with name {0}. An animation sequence cannot contain more than 65535 tracks"), FText::FromName(BoneName));
			return INDEX_NONE;
		}
		else
		{
			// Determine correct index to do insertion at
			int32 BoneIndex = INDEX_NONE;

			if (const UAnimSequence* AnimationSequence = Model->GetAnimationSequence())
			{
				if (const USkeleton* Skeleton = AnimationSequence->GetSkeleton())
				{
					BoneIndex = Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneName);

					if (BoneIndex == INDEX_NONE)
					{
						ReportErrorf(LOCTEXT("UnableToFindBoneIndexWarning", "Unable to retrieve bone index for track: {0}"), FText::FromName(BoneName));
					}
				}
				else
				{
					ReportError(LOCTEXT("UnableToGetOuterSkeletonError", "Unable to retrieve Skeleton for outer Animation Sequence"));
				}
			}
			else
			{
				ReportError(LOCTEXT("UnableToGetOuterAnimSequenceError", "Unable to retrieve outer Animation Sequence"));
			}

			if (BoneIndex != INDEX_NONE || IgnoreSkeletonValidation())
			{
				IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("InsertBoneTrack", "Inserting Animation Data Track"), bShouldTransact);
				
				FAnimationTrackAddedPayload Payload;
				Payload.Name = BoneName;

				if(!AddBoneControl(BoneName))
				{
					ReportErrorf(LOCTEXT("FailedToAddBoneControlElement", "Failed to add Bone control with name {0}"), FText::FromName(BoneName));
				}
				
				ConditionalAction<UE::Anim::FRemoveTrackAction>(bShouldTransact, BoneName);
				Model->GetNotifier().Notify<FAnimationTrackAddedPayload>(EAnimDataModelNotifyType::TrackAdded, Payload);

				return 0;
			}
		}
	}
	else
	{
		ReportWarningf(LOCTEXT("TrackNameAlreadyExistsWarning", "Track with name {0} already exists"), FText::FromName(BoneName));
	}
	
	return INDEX_NONE;
}

bool UAnimSequencerController::RemoveBoneTrack(FName BoneName, bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return false;
	}

	if (Model->IsValidBoneTrackName(BoneName))
	{
		IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
		FTransaction Transaction = ConditionalTransaction(LOCTEXT("RemoveBoneTrack", "Removing Animation Data Track"), bShouldTransact);

		if (bShouldTransact)
		{
			TArray<FTransform> BoneTransforms;
			Model->GetBoneTrackTransforms(BoneName, BoneTransforms);
			ConditionalAction<UE::Anim::FAddTrackAction>(bShouldTransact, BoneName, MoveTemp(BoneTransforms));			
		}
	
		if(!RemoveBoneControl(BoneName))
		{
			ReportError(LOCTEXT("FailedToRemoveBoneControl", "Failed to remove bone control"));
		}

		FAnimationTrackRemovedPayload Payload;
		Payload.Name = BoneName;

		Model->GetNotifier().Notify(EAnimDataModelNotifyType::TrackRemoved, Payload);

		return true;
	}
	else
	{
		ReportWarningf(LOCTEXT("UnableToFindTrackWarning", "Could not find track with name {0}"), FText::FromName(BoneName));
	}

	return false;
}

void UAnimSequencerController::RemoveAllBoneTracks(bool bShouldTransact /*= true*/)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return;
	}
	
	TArray<FName> TrackNames;
	Model->GetBoneTrackNames(TrackNames);

	if (TrackNames.Num())
	{
		FBracket Bracket = ConditionalBracket(LOCTEXT("RemoveAllBoneTracks", "Removing all Animation Data Tracks"), bShouldTransact);
		for (const FName& TrackName : TrackNames)
		{
			RemoveBoneTrack(TrackName, bShouldTransact);
		}
	}	
}

bool UAnimSequencerController::SetBoneTrackKeys(FName BoneName, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys,
	bool bShouldTransact)
{
	if (!ModelInterface->GetAnimationSequence())
	{
		return false;
	}

	// Validate key format
	const int32 MaxNumKeys = FMath::Max(FMath::Max(PositionalKeys.Num(), RotationalKeys.Num()), ScalingKeys.Num());

	if (MaxNumKeys > 0)
	{
		const bool bValidPosKeys = PositionalKeys.Num() == MaxNumKeys;
		const bool bValidRotKeys = RotationalKeys.Num() == MaxNumKeys;
		const bool bValidScaleKeys = ScalingKeys.Num() == MaxNumKeys;

		if (bValidPosKeys && bValidRotKeys && bValidScaleKeys)
		{
			if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
			{
				if (UControlRig* ControlRig = Model->GetControlRig())
				{
					if (URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
					{
						const FRigElementKey BoneKey(URigHierarchy::GetSanitizedName(BoneName), ERigElementType::Bone);
						const FRigElementKey BoneNameControlKey(UFKControlRig::GetControlName(BoneKey.Name, ERigElementType::Bone), ERigElementType::Control);
						
						const bool bContainsBone = Hierarchy->Contains(BoneKey);
						const bool bContainsBoneControl = Hierarchy->Contains(BoneNameControlKey);
						if (bContainsBone && bContainsBoneControl)
						{
							IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
							FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetTrackKeysTransaction", "Setting Animation Data Track keys"), bShouldTransact);

							if (bShouldTransact)
							{
								TArray<FTransform> BoneTransforms;
								Model->GetBoneTrackTransforms(BoneName, BoneTransforms);
								ConditionalAction<UE::Anim::FSetTrackKeysAction>(bShouldTransact, BoneName, BoneTransforms);							
							}

							if(!SetBoneCurveKeys(BoneName, PositionalKeys, RotationalKeys, ScalingKeys))
							{
								ReportError(LOCTEXT("FailedToSetBoneCurveKeys", "Failed to set bone curve keys"));
							}

							FAnimationTrackChangedPayload Payload;
							Payload.Name = BoneName;

							Model->GetNotifier().Notify(EAnimDataModelNotifyType::TrackChanged, Payload);

							return true;
						}
					
					}
					else
					{
						ReportWarningf(LOCTEXT("InvalidTrackNameWarning", "Track with name {0} does not exist"), FText::FromName(BoneName));
					}
				}
			}
		}
		else
		{
			ReportErrorf(LOCTEXT("InvalidTrackKeyDataError", "Invalid track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
		}
	}
	else
	{
		ReportErrorf(LOCTEXT("MissingTrackKeyDataError", "Missing track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
	}

	return false;
}

bool UAnimSequencerController::SetBoneTrackKeys(FName BoneName, const TArray<FVector>& PositionalKeys, const TArray<FQuat>& RotationalKeys, const TArray<FVector>& ScalingKeys, bool bShouldTransact /*= true*/)
{
	TArray<FVector3f> FloatPositionalKeys;
	TArray<FQuat4f> FloatRotationalKeys;
	TArray<FVector3f> FloatScalingKeys;

	FloatPositionalKeys.Reserve(PositionalKeys.Num());
	FloatRotationalKeys.Reserve(RotationalKeys.Num());
	FloatScalingKeys.Reserve(ScalingKeys.Num());

	Algo::Transform(PositionalKeys, FloatPositionalKeys, [](const FVector& V) { return FVector3f(V); });
	Algo::Transform(RotationalKeys, FloatRotationalKeys, [](const FQuat& Q) { return FQuat4f(Q); });
	Algo::Transform(ScalingKeys, FloatScalingKeys, [](const FVector& V) { return FVector3f(V); });

	return SetBoneTrackKeys(BoneName, FloatPositionalKeys, FloatRotationalKeys, FloatScalingKeys, bShouldTransact);
}

static int32 DiscreteInclusiveLower(const TRange<int32>& InRange)
{
	check(!InRange.GetLowerBound().IsOpen());

	// Add one for exclusive lower bounds since they start on the next subsequent frame
	static constexpr int32 Offsets[]   = { 0, 1 };
	const int32        OffsetIndex = (int32)InRange.GetLowerBound().IsExclusive();

	return InRange.GetLowerBound().GetValue() + Offsets[OffsetIndex];
}

static int32 DiscreteExclusiveUpper(const TRange<int32>& InRange)
{
	check(!InRange.GetUpperBound().IsOpen());

	// Add one for inclusive upper bounds since they finish on the next subsequent frame
	static constexpr int32 Offsets[]   = { 0, 1 };
	const int32        OffsetIndex = (int32)InRange.GetUpperBound().IsInclusive();

	return InRange.GetUpperBound().GetValue() + Offsets[OffsetIndex];
}

bool UAnimSequencerController::UpdateBoneTrackKeys(FName BoneName, const FInt32Range& KeyRangeToSet, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys, bool bShouldTransact)
{
	if (!CheckOuterClass(UAnimSequence::StaticClass()))
	{
		return false;
	}
	
	// Validate key format
	const int32 MaxNumKeys = FMath::Max(FMath::Max(PositionalKeys.Num(), RotationalKeys.Num()), ScalingKeys.Num());
	const int32 RangeMin = DiscreteInclusiveLower(KeyRangeToSet);
	const int32 RangeMax = DiscreteExclusiveUpper(KeyRangeToSet);
	if (MaxNumKeys > 0)
	{
		const bool bValidPosKeys = PositionalKeys.Num() == MaxNumKeys;
		const bool bValidRotKeys = RotationalKeys.Num() == MaxNumKeys;
		const bool bValidScaleKeys = ScalingKeys.Num() == MaxNumKeys;

		if (bValidPosKeys && bValidRotKeys && bValidScaleKeys)
		{
			const int32 NumKeysToSet = RangeMax - RangeMin;
			if (NumKeysToSet == MaxNumKeys)
			{
				if (Model->IsValidBoneTrackName(BoneName))
				{
					const FInt32Range TrackKeyRange(0, Model->GetNumberOfKeys());
					if(TrackKeyRange.Contains(KeyRangeToSet))
					{
						IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
						FTransaction Transaction = ConditionalTransaction(LOCTEXT("SetTrackKeysRangeTransaction", "Setting Animation Data Track keys"), bShouldTransact);

						if (bShouldTransact)
						{
							TArray<FTransform> BoneTransforms;
							Model->GetBoneTrackTransforms(BoneName, BoneTransforms);
							ConditionalAction<UE::Anim::FSetTrackKeysAction>(bShouldTransact, BoneName, BoneTransforms);
						}
						
						TArray<FTransform> TransformKeys;
						TArray<float> TimeValues;
					
						const FFrameRate& FrameRate = ModelInterface->GetFrameRate();
						TransformKeys.Reserve(PositionalKeys.Num());
						TimeValues.Reserve(PositionalKeys.Num());
						int32 KeyIndex = 0;
						for (int32 RangeIndex = RangeMin; RangeIndex < RangeMax; ++RangeIndex, ++KeyIndex)
						{
							TransformKeys.Add(FTransform(FQuat(RotationalKeys[KeyIndex]), FVector(PositionalKeys[KeyIndex]), FVector(ScalingKeys[KeyIndex])));
							TimeValues.Add(static_cast<float>(FrameRate.AsSeconds(RangeIndex)));
						}
					
						if(!UpdateBoneCurveKeys(BoneName, TransformKeys, TimeValues))
						{
							ReportError(LOCTEXT("FailedToUpdateBoneCurveKeys", "Failed to update bone curve keys"));
						}
						
						FAnimationTrackChangedPayload Payload;
						Payload.Name = BoneName;
						
						Model->GetNotifier().Notify(EAnimDataModelNotifyType::TrackChanged, Payload);

						return true;
					}
					else
					{
						ReportWarningf(LOCTEXT("InvalidTrackSetFrameRange", "Range of to-be-set bone track (with name {0}) keys does not overlap existing range"), FText::FromName(BoneName));
					}
				}
				else
				{
					ReportWarningf(LOCTEXT("InvalidTrackNameWarning", "Track with name {0} does not exist"), FText::FromName(BoneName));
				}
			}
			else
			{
				ReportErrorf(LOCTEXT("InvalidKeyIndexRangeError",
								 "Invalid key index range bound [{0}, {1}], expected to equal the size of the positional, rotational and scaling keys {2}"),
								 FText::AsNumber(RangeMin),
								 FText::AsNumber(RangeMax),
								 FText::AsNumber(MaxNumKeys));
			}	
		}
		else
		{
			ReportErrorf(LOCTEXT("InvalidTrackKeyDataError", "Invalid track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
		}
	}
	else
	{
		ReportErrorf(LOCTEXT("MissingTrackKeyDataError", "Missing track key data, expected uniform data: number of positional keys {0}, number of rotational keys {1}, number of scaling keys {2}"), FText::AsNumber(PositionalKeys.Num()), FText::AsNumber(RotationalKeys.Num()), FText::AsNumber(ScalingKeys.Num()));
	}

	return false;
}

bool UAnimSequencerController::UpdateBoneTrackKeys(FName BoneName, const FInt32Range& KeyRangeToSet, const TArray<FVector>& PositionalKeys, const TArray<FQuat>& RotationalKeys, const TArray<FVector>& ScalingKeys, bool bShouldTransact)
{
	TArray<FVector3f> FloatPositionalKeys;
	TArray<FQuat4f> FloatRotationalKeys;
	TArray<FVector3f> FloatScalingKeys;
	FloatPositionalKeys.Reserve(PositionalKeys.Num());
	FloatRotationalKeys.Reserve(RotationalKeys.Num());
	FloatScalingKeys.Reserve(ScalingKeys.Num());

	Algo::Transform(PositionalKeys, FloatPositionalKeys, [](const FVector& V) { return FVector3f(V); });
	Algo::Transform(RotationalKeys, FloatRotationalKeys, [](const FQuat& Q) { return FQuat4f(Q); });
	Algo::Transform(ScalingKeys, FloatScalingKeys, [](const FVector& V) { return FVector3f(V); });

	return UpdateBoneTrackKeys(BoneName, KeyRangeToSet, FloatPositionalKeys, FloatRotationalKeys, FloatScalingKeys, bShouldTransact);
}

void UAnimSequencerController::ResizeCurves(float NewLength, bool bInserted, float T0, float T1, bool bShouldTransact /*= true*/)
{
	FBracket Bracket = ConditionalBracket(LOCTEXT("ResizeCurves", "Resizing all Curves"), bShouldTransact);

	for (FFloatCurve& Curve : Model->LegacyCurveData.FloatCurves)
	{
		FFloatCurve ResizedCurve = Curve;
		ResizedCurve.Resize(NewLength, bInserted, T0, T1);
		SetCurveKeys(FAnimationCurveIdentifier(Curve.GetName(), ERawCurveTrackTypes::RCT_Float), ResizedCurve.FloatCurve.GetConstRefOfKeys(), bShouldTransact);
	}

	for (FTransformCurve& Curve : Model->LegacyCurveData.TransformCurves)
	{
		FTransformCurve ResizedCurve = Curve;
		for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
		{
			const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
			FVectorCurve& SubCurve = *ResizedCurve.GetVectorCurveByIndex(SubCurveIndex);
			for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
			{
				const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
				FAnimationCurveIdentifier TargetCurveIdentifier = FAnimationCurveIdentifier(Curve.GetName(), ERawCurveTrackTypes::RCT_Transform);
				UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
				
				FRichCurve& ChannelCurve = SubCurve.FloatCurves[ChannelIndex];
				ChannelCurve.ReadjustTimeRange(0, NewLength, bInserted, T0, T1);
				SetCurveKeys(TargetCurveIdentifier, ChannelCurve.GetConstRefOfKeys(), bShouldTransact);
			}
		}
	}
}

void UAnimSequencerController::ResizeAttributes(float NewLength, bool bInserted, float T0, float T1, bool bShouldTransact)
{
	FBracket Bracket = ConditionalBracket(LOCTEXT("ResizeAttributes", "Resizing all Attributes"), bShouldTransact);

	for (FAnimatedBoneAttribute& Attribute : Model->AnimatedBoneAttributes)
	{
		FAttributeCurve ResizedCurve = Attribute.Curve;
		ResizedCurve.ReadjustTimeRange(0, NewLength, bInserted, T0, T1);

		// Generate arrays necessary for API
		TArray<float> Times;
		TArray<const void*> Values;
		const TArray<FAttributeKey>& Keys = ResizedCurve.GetConstRefOfKeys();
		for (int32 KeyIndex = 0; KeyIndex < ResizedCurve.GetNumKeys(); ++KeyIndex)
		{
			Times.Add(Keys[KeyIndex].Time);
			Values.Add(Keys[KeyIndex].GetValuePtr<void>());
		}
		
		SetAttributeKeys(Attribute.Identifier, Times, Values, Attribute.Curve.GetScriptStruct(), bShouldTransact);
	}
}

bool UAnimSequencerController::AddAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, bool bShouldTransact /*= true*/)
{
	if (AttributeIdentifier.IsValid())
	{
		const FAnimatedBoneAttribute* const ExistingAttribute = Model->AnimatedBoneAttributes.FindByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute) -> bool
		{
			return Attribute.Identifier.GetName() == AttributeIdentifier.GetName() && Attribute.Identifier.GetBoneName() == AttributeIdentifier.GetBoneName();
		});

		if (!ExistingAttribute)
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("AddAttribute", "Adding Animated Bone Attribute"), bShouldTransact);

			FAnimatedBoneAttribute& Attribute = Model->AnimatedBoneAttributes.AddDefaulted_GetRef();
			Attribute.Identifier = AttributeIdentifier;

			Attribute.Curve.SetScriptStruct(AttributeIdentifier.GetType());
		
			ConditionalAction<UE::Anim::FRemoveAtributeAction>(bShouldTransact,  AttributeIdentifier);

			FAttributeAddedPayload Payload;
			Payload.Identifier = AttributeIdentifier;
			Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeAdded, Payload);

			return true;
		}
		else
		{
			ReportErrorf(LOCTEXT("AttributeAlreadyExists", "Attribute identifier with matching name already exists: {0} {1} ({2}) {3}"),
				FText::FromName(ExistingAttribute->Identifier.GetName()),
				FText::FromName(ExistingAttribute->Identifier.GetBoneName()),
				FText::AsNumber(ExistingAttribute->Identifier.GetBoneIndex()),
				FText::FromName(ExistingAttribute->Identifier.GetType()->GetFName()));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidAttributeIdentifier", "Invalid attribute identifier provided"));
	}

	return false;
}

bool UAnimSequencerController::RemoveAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, bool bShouldTransact /*= true*/)
{
	if (AttributeIdentifier.IsValid())
	{
		const int32 AttributeIndex = Model->AnimatedBoneAttributes.IndexOfByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute) -> bool
		{
			return Attribute.Identifier == AttributeIdentifier;
		});

		if (AttributeIndex != INDEX_NONE)
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			FTransaction Transaction = ConditionalTransaction(LOCTEXT("RemoveAttribute", "Removing Animated Bone Attribute"), bShouldTransact);

			ConditionalAction<UE::Anim::FAddAtributeAction>(bShouldTransact,  Model->AnimatedBoneAttributes[AttributeIndex]);

			Model->AnimatedBoneAttributes.RemoveAtSwap(AttributeIndex);
			
			FAttributeRemovedPayload Payload;
			Payload.Identifier = AttributeIdentifier;
			Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeRemoved, Payload);

			return true;
		}
		else
		{
			ReportErrorf(LOCTEXT("AttributeNotFound", "Attribute identifier provided was not found: {0} {1} ({2}) {3}"),
				FText::FromName(AttributeIdentifier.GetName()), FText::FromName(AttributeIdentifier.GetBoneName()), FText::AsNumber(AttributeIdentifier.GetBoneIndex()), FText::FromName(AttributeIdentifier.GetType()->GetFName()));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidAttributeIdentifier", "Invalid attribute identifier provided"));
	}

	return false;
}

int32 UAnimSequencerController::RemoveAllAttributesForBone(const FName& BoneName, bool bShouldTransact)
{
	int32 NumRemovedAttributes = 0;

	// Generate list of attribute identifiers, matching the bone name, for removal
	TArray<FAnimationAttributeIdentifier> Identifiers;
	Algo::TransformIf(Model->AnimatedBoneAttributes, Identifiers,
		[BoneName](const FAnimatedBoneAttribute& Attribute) -> bool
		{
			return Attribute.Identifier.GetBoneName() == BoneName;
		},
		[](const FAnimatedBoneAttribute& Attribute)
		{
			return Attribute.Identifier;
		}
	);

	if (Identifiers.Num())
	{
		FBracket Bracket = ConditionalBracket(LOCTEXT("RemoveAllAttributesForBone", "Removing all Attributes for Bone"), bShouldTransact);
		for (const FAnimationAttributeIdentifier& Identifier : Identifiers)
		{
			NumRemovedAttributes += RemoveAttribute(Identifier, bShouldTransact) ? 1 : 0;
		}
	}	

	return NumRemovedAttributes;
}

int32 UAnimSequencerController::RemoveAllAttributes(bool bShouldTransact)
{
	int32 NumRemovedAttributes = 0;

	TArray<FAnimationAttributeIdentifier> Identifiers;
	Algo::Transform(Model->AnimatedBoneAttributes, Identifiers, [](const FAnimatedBoneAttribute& Attribute)
	{
		return Attribute.Identifier;
	});

	if (Identifiers.Num())
	{
		FBracket Bracket = ConditionalBracket(LOCTEXT("RemoveAllAttributes", "Removing all Attributes"), bShouldTransact);
		for (const FAnimationAttributeIdentifier& Identifier : Identifiers)
		{
			NumRemovedAttributes += RemoveAttribute(Identifier, bShouldTransact) ? 1: 0;
		}
	}

	return NumRemovedAttributes;
}

bool UAnimSequencerController::SetAttributeKey_Internal(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, const void* KeyValue, const UScriptStruct* TypeStruct, bool bShouldTransact /*= true*/)
{
	if (AttributeIdentifier.IsValid())
	{
		if (KeyValue)
		{
			FAnimatedBoneAttribute* AttributePtr = Model->AnimatedBoneAttributes.FindByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute)
			{
				return Attribute.Identifier == AttributeIdentifier;
			});

			if (AttributePtr)
			{
				if (TypeStruct == AttributePtr->Identifier.GetType())
				{
					IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
					FTransaction Transaction = ConditionalTransaction(LOCTEXT("SettingAttributeKey", "Setting Animated Bone Attribute key"), bShouldTransact);

					FAttributeCurve& Curve = AttributePtr->Curve;
					const FKeyHandle KeyHandle = Curve.FindKey(Time);
					// In case the key does not yet exist one will be added, and thus the undo is a remove
					if (KeyHandle == FKeyHandle::Invalid())
					{
						ConditionalAction<UE::Anim::FRemoveAtributeKeyAction>(bShouldTransact,  AttributeIdentifier, Time);
						Curve.UpdateOrAddTypedKey(Time, KeyValue, TypeStruct);
					}
					// In case the key does exist it will be updated , and thus the undo is a revert to the current value
					else
					{
						ConditionalAction<UE::Anim::FSetAtributeKeyAction>(bShouldTransact,  AttributeIdentifier, Curve.GetKey(KeyHandle));
						Curve.UpdateOrAddTypedKey(Time, KeyValue, TypeStruct);
					}

					FAttributeChangedPayload Payload;
					Payload.Identifier = AttributeIdentifier;
					Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeChanged, Payload);

					return true;
				}
				else
				{
					ReportErrorf(LOCTEXT("AttributeTypeDoesNotMatchKeyType", "Key type does not match attribute: {0} {1}"),
						FText::FromName(AttributePtr->Identifier.GetType()->GetFName()), FText::FromName(TypeStruct->GetFName()));
				}
			}
			else
			{
				ReportErrorf(LOCTEXT("AttributeNotFound", "Attribute identifier provided was not found: {0} {1} ({2}) {3}"),
					FText::FromName(AttributeIdentifier.GetName()), FText::FromName(AttributeIdentifier.GetBoneName()), FText::AsNumber(AttributeIdentifier.GetBoneIndex()), FText::FromName(AttributeIdentifier.GetType()->GetFName()));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidAttributeKey", "Invalid attribute key value provided"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidAttributeIdentifier", "Invalid attribute identifier provided"));
	}

	return false;
}

bool UAnimSequencerController::SetAttributeKeys_Internal(const FAnimationAttributeIdentifier& AttributeIdentifier, TArrayView<const float> Times, TArrayView<const void*> KeyValues, const UScriptStruct* TypeStruct, bool bShouldTransact)
{
	if (AttributeIdentifier.IsValid())
	{
		if (Times.Num() == KeyValues.Num())
		{
			FAnimatedBoneAttribute* AttributePtr = Model->AnimatedBoneAttributes.FindByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute)
			{
				return Attribute.Identifier == AttributeIdentifier;
			});

			if (AttributePtr)
			{
				if (TypeStruct == AttributePtr->Identifier.GetType())
				{
					IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
					FTransaction Transaction = ConditionalTransaction(LOCTEXT("SettingAttributeKeys", "Setting Animated Bone Attribute keys"), bShouldTransact);

					FAnimatedBoneAttribute& Attribute = *AttributePtr;

					ConditionalAction<UE::Anim::FSetAtributeKeysAction>(bShouldTransact,  Attribute);
			
					Attribute.Curve.SetKeys(Times, KeyValues);

					FAttributeChangedPayload Payload;
					Payload.Identifier = AttributeIdentifier;
					Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeChanged, Payload);

					return true;
				}
				else
				{
					ReportErrorf(LOCTEXT("AttributeTypeDoesNotMatchKeyType", "Key type does not match attribute: {0} {1}"),
						FText::FromName(AttributePtr->Identifier.GetType()->GetFName()), FText::FromName(TypeStruct->GetFName()));
				}
			}
			else
			{
				ReportErrorf(LOCTEXT("AttributeNotFound", "Attribute identifier provided was not found: {0} {1} ({2}) {3}"),
					FText::FromName(AttributeIdentifier.GetName()), FText::FromName(AttributeIdentifier.GetBoneName()), FText::AsNumber(AttributeIdentifier.GetBoneIndex()), FText::FromName(AttributeIdentifier.GetType()->GetFName()));
			}
		}
		else
		{
			ReportErrorf(LOCTEXT("AttributeKeysMismatch", "Non matching number of key time/values: time entries {0} key entries {1}"),
				FText::AsNumber(Times.Num()), FText::AsNumber(KeyValues.Num()));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidAttributeIdentifier", "Invalid attribute identifier provided"));
	}

	return false;
}


bool UAnimSequencerController::RemoveAttributeKey(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, bool bShouldTransact /*= true*/)
{
	if (AttributeIdentifier.IsValid())
	{
		FAnimatedBoneAttribute* AttributePtr = Model->AnimatedBoneAttributes.FindByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute)
		{
			return Attribute.Identifier == AttributeIdentifier;
		});

		if (AttributePtr)
		{
			FAttributeCurve& Curve = AttributePtr->Curve;
			const FKeyHandle KeyHandle = Curve.FindKey(Time);

			if (KeyHandle != FKeyHandle::Invalid())
			{
				IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
				FTransaction Transaction = ConditionalTransaction(LOCTEXT("RemovingAttributeKey", "Removing Animated Bone Attribute key"), bShouldTransact);

				ConditionalAction<UE::Anim::FAddAtributeKeyAction>(bShouldTransact,  AttributeIdentifier, Curve.GetKey(KeyHandle));

				Curve.DeleteKey(KeyHandle);

				FAttributeAddedPayload Payload;
				Payload.Identifier = AttributeIdentifier;
				Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeChanged, Payload);

				return true;
			}
			else
			{
				ReportWarning(LOCTEXT("AttributeKeyNotFound", "Attribute does not contain key for provided time"));
			}	
		}
		else
		{
			ReportErrorf(LOCTEXT("AttributeNotFound", "Attribute identifier provided was not found: {0} {1} ({2}) {3}"),
				FText::FromName(AttributeIdentifier.GetName()), FText::FromName(AttributeIdentifier.GetBoneName()), FText::AsNumber(AttributeIdentifier.GetBoneIndex()), FText::FromName(AttributeIdentifier.GetType()->GetFName()));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidAttributeIdentifier", "Invalid attribute identifier provided"));
	}

	return false;
}

bool UAnimSequencerController::DuplicateAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, const FAnimationAttributeIdentifier& NewAttributeIdentifier, bool bShouldTransact)
{
	ValidateModel();

	if (AttributeIdentifier.IsValid() && NewAttributeIdentifier.IsValid())
	{
		if (AttributeIdentifier.GetType() == NewAttributeIdentifier.GetType())
		{
			const FAnimatedBoneAttribute* ExistingAttributePtr = Model->FindAttribute(NewAttributeIdentifier);
			if (ExistingAttributePtr == nullptr)
			{
				if(const FAnimatedBoneAttribute* AttributePtr = Model->FindAttribute(AttributeIdentifier))
				{
					IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
					FTransaction Transaction = ConditionalTransaction(LOCTEXT("DuplicateAttribute", "Duplicating Animation Attribute"), bShouldTransact);

					FAnimatedBoneAttribute DuplicateAttribute;
					DuplicateAttribute.Identifier = NewAttributeIdentifier;
					DuplicateAttribute.Curve = AttributePtr->Curve;
					Model->AnimatedBoneAttributes.Add(DuplicateAttribute);

					FAttributeAddedPayload Payload;
					Payload.Identifier = NewAttributeIdentifier;
					Model->GetNotifier().Notify(EAnimDataModelNotifyType::AttributeAdded, Payload);

					ConditionalAction<UE::Anim::FRemoveAtributeAction>(bShouldTransact,  NewAttributeIdentifier);
					
					return true;
				}
				else
				{
					ReportWarningf(LOCTEXT("AttributeNameToDuplicateNotFoundWarning", "Could not find attribute with name {0} and type {1} for duplication"), FText::FromName(AttributeIdentifier.GetName()), FText::FromString(AttributeIdentifier.GetType()->GetName()));
				}
			}
			else
			{
				ReportWarningf(LOCTEXT("ExistingAttributeWarning", "Attribute with name {0} already exists"), FText::FromName(AttributeIdentifier.GetName()));
			}
		}
		else
		{
			ReportWarningf(LOCTEXT("InvalidAttributeTypeWarning", "Attribute types do not match: {0} ({1})"), FText::FromString(AttributeIdentifier.GetType()->GetName()), FText::FromString(AttributeIdentifier.GetType()->GetName()));
		}
	}
	else
	{
		ReportWarningf(LOCTEXT("InvalidAttributeIdentifierWarning", "Invalid attribute identifier(s) provided: {0} and/or {1}"), FText::FromName(AttributeIdentifier.GetName()), FText::FromName(AttributeIdentifier.GetName()));
	}

	return false;
}

void UAnimSequencerController::RemoveUnusedControlsAndCurves() const
{
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		if (UControlRig* ControlRig = Model->GetControlRig())
		{
			if (URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
			{
				TArray<FRigElementKey> ElementsToRemove;
				
				// Remove any curve/bone controls from the hierarchy for which no data is currently keyed
				Hierarchy->ForEach<FRigControlElement>([TransformCurves=Section->GetTransformParameterNamesAndCurves(), ScalarCurves=Section->GetScalarParameterNamesAndCurves(),&ElementsToRemove](const FRigControlElement* ControlElement) -> bool
				{
					if(!(TransformCurves.ContainsByPredicate([ControlElement](const FTransformParameterNameAndCurves& TransformParameter)
							{
								return TransformParameter.ParameterName == ControlElement->GetFName();
							})
							||
							ScalarCurves.ContainsByPredicate([ControlElement](const FScalarParameterNameAndCurve& ScalarParameter)
							{
								return ScalarParameter.ParameterName == ControlElement->GetFName();
							}))
					)
					{
						ElementsToRemove.Add(ControlElement->GetKey());
					}
						
					return true;
				});
				
				// Remove any curves from the hierarchy for which no data is currently keyed				
				const TArray<FFloatCurve>& FloatCurves = Model->GetFloatCurves();
				Hierarchy->ForEach<FRigCurveElement>([FloatCurves, &ElementsToRemove](const FRigCurveElement* CurveElement) -> bool
				{
					if(!FloatCurves.ContainsByPredicate([CurveElement](const FFloatCurve& FloatCurve)
					{
						return FloatCurve.GetName() == CurveElement->GetFName();
					}))
					{						
						ElementsToRemove.Add(CurveElement->GetKey());
					}

					return true;
				});

				IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
				for (const FRigElementKey& Key : ElementsToRemove)
				{
					Hierarchy->GetController()->RemoveElement(Key);
				}
			}
		}
	}
}

void UAnimSequencerController::UpdateWithSkeleton(USkeleton* TargetSkeleton, bool bShouldTransact)
{
	if (ModelInterface->HasBeenPopulated())
	{
		OpenBracket(LOCTEXT("SettingNewskeleton", "Updating Skeleton for Animation Data Model"), bShouldTransact);
		{
			// Work out if any work has to be performed.
			TArray<FName> BonesToBeAdded;
			TArray<FName> BonesToBeRemoved;
			TArray<FName> CurvesToBeRemoved;

			if(TargetSkeleton)
			{
			    const FReferenceSkeleton& ReferenceSkeleton = TargetSkeleton->GetReferenceSkeleton();
			    IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			    if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
			    {
			    	// find bones needing to be REMOVED
				    const TArray<FTransformParameterNameAndCurves>& BoneTrackCurves = Section->GetTransformParameterNamesAndCurves();
				    for (const FTransformParameterNameAndCurves& BoneCurve : BoneTrackCurves)
				    {
					    const FName ExpectedBoneName = UFKControlRig::GetControlTargetName(BoneCurve.ParameterName, ERigElementType::Bone);
					    
					    if (ReferenceSkeleton.FindBoneIndex(ExpectedBoneName) == INDEX_NONE)
					    {
						    // Bone no longer exists on skeleton
						    BonesToBeRemoved.Add(ExpectedBoneName);
					    }
				    }

			    	// find bones needing to be ADDED
				    const TArray<FMeshBoneInfo>& RawRefBones = ReferenceSkeleton.GetRawRefBoneInfo();
			    	for (const FMeshBoneInfo& RawRefBone : RawRefBones)
			    	{
			    		bool bFoundBoneTrack = false;
			    		for (const FTransformParameterNameAndCurves& BoneCurve : BoneTrackCurves)
			    		{
			    			if (BoneCurve.ParameterName == RawRefBone.Name)
			    			{
			    				bFoundBoneTrack = true;
			    				break;
			    			}
			    		}
			    		if (!bFoundBoneTrack)
			    		{
			    			BonesToBeAdded.Add(RawRefBone.Name);
			    		}
			    	}

			    	// find curves needing to be REMOVED
				    TArray<FName> SkeletonCurveNames;
				    TargetSkeleton->GetCurveMetaDataNames(SkeletonCurveNames);
				    const TArray<FScalarParameterNameAndCurve>& SectionCurves = Section->GetScalarParameterNamesAndCurves();
				    for (const FScalarParameterNameAndCurve& SectionCurve : SectionCurves)
				    {
					    const FName ExpectedCurveName = UFKControlRig::GetControlTargetName(SectionCurve.ParameterName, ERigElementType::Curve);
					    if (!SkeletonCurveNames.Contains(ExpectedCurveName))
					    {
						    // Curve no longer exists on skeleton
						    CurvesToBeRemoved.Add(ExpectedCurveName);
					    }
				    }
			    }
			}

			if (BonesToBeAdded.Num() || BonesToBeRemoved.Num() || CurvesToBeRemoved.Num())
			{
				// (re-)generate the rig hierarchy
				Model->InitializeFKControlRig(CastChecked<UFKControlRig>(Model->GetControlRig()), TargetSkeleton, true);
				
				// Remove any bone tracks that do not exist in the new hierarchy
				RemoveBoneTracksMissingFromSkeleton(TargetSkeleton, bShouldTransact);

				// Remove any control/curves which were created in InitializeFKControlRig which are no longer keyed
				RemoveUnusedControlsAndCurves();

				// Forcefully re-generate legacy data structures
				Model->RegenerateLegacyCurveData();
			}
			
			// Notify of skeleton change
			Model->GetNotifier().Notify(EAnimDataModelNotifyType::SkeletonChanged);
		}
		CloseBracket(bShouldTransact);
	}	
}

void UAnimSequencerController::PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> InModel)
{
	EnsureModelIsInitialized();
	
	OpenBracket(LOCTEXT("PopulateWithExistingModel", "Copying Data from Legacy Animation Data Model"));
	if (InModel.GetObject() && InModel.GetObject()->IsA<UAnimDataModel>())
	{
		IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);	
		Model->CachedRawDataGUID = InModel->GenerateGuid();
		
		// Copy over frame-rate from model
		const FFrameRate ModelFrameRate = InModel->GetFrameRate();

		SetFrameRate(ModelFrameRate, false);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const TArray<FBoneAnimationTrack>& BoneTracks = InModel->GetBoneAnimationTracks();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		const bool bHasBoneTracks = BoneTracks.Num() > 0;

		FFrameNumber NumberOfFrames = InModel->GetNumberOfFrames();

		const int32 MaxNumberOfBoneKeys = [&BoneTracks]()
		{
			int32 NumKeys = 0;

			for (const FBoneAnimationTrack& Track : BoneTracks)
			{
				NumKeys = FMath::Max(NumKeys, Track.InternalTrackData.PosKeys.Num());
				NumKeys = FMath::Max(NumKeys, Track.InternalTrackData.RotKeys.Num());
				NumKeys = FMath::Max(NumKeys, Track.InternalTrackData.ScaleKeys.Num());
			}
			
			return NumKeys;
		}();

		if (bHasBoneTracks && MaxNumberOfBoneKeys != (NumberOfFrames.Value + 1))
		{			
			ReportWarningf(LOCTEXT("NonMatchingNumberOfKeysAndData", "Mismatch between number of stored bone animation keys {0} and value stored on the DataModel {1}, biggest found number of keys will be used."), FText::AsNumber(MaxNumberOfBoneKeys), FText::AsNumber(NumberOfFrames.Value + 1));
			NumberOfFrames = MaxNumberOfBoneKeys > NumberOfFrames.Value + 1 ? MaxNumberOfBoneKeys - 1 : NumberOfFrames.Value;
		}
		
		SetNumberOfFrames(NumberOfFrames, false);

		if (const UAnimSequence* AnimSequence = Model->GetAnimationSequence())
		{
			const USkeleton* Skeleton = AnimSequence->GetSkeleton();
			if (Skeleton)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_PopulateBoneTrack);
				for (const FBoneAnimationTrack& BoneTrack : BoneTracks)
				{
					// It could be that the existing model has out-of-date track data for bones previously removed from the Skeleton
					if (Skeleton->GetReferenceSkeleton().FindBoneIndex(BoneTrack.Name) != INDEX_NONE)
					{
						AddBoneCurve(BoneTrack.Name, false);
						SetBoneTrackKeys(BoneTrack.Name, BoneTrack.InternalTrackData.PosKeys, BoneTrack.InternalTrackData.RotKeys, BoneTrack.InternalTrackData.ScaleKeys, false);
					}
				}
			}

			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_PopulateCurves);
				for (const FFloatCurve& Curve : InModel->GetCurveData().FloatCurves)
				{
					const FAnimationCurveIdentifier CurveId = Skeleton ? UAnimationCurveIdentifierExtensions::FindCurveIdentifier(Skeleton, Curve.GetName(), ERawCurveTrackTypes::RCT_Float) : FAnimationCurveIdentifier(Curve.GetName(), ERawCurveTrackTypes::RCT_Float);
					if (CurveId.IsValid() || IgnoreSkeletonValidation())
					{
						AddCurve(CurveId, Curve.GetCurveTypeFlags(), false);
						FCurveAttributes Attributes;
						Attributes.SetPreExtrapolation(Curve.FloatCurve.PreInfinityExtrap);
						Attributes.SetPostExtrapolation(Curve.FloatCurve.PostInfinityExtrap);						
						SetCurveAttributes(CurveId, Attributes, false);
						SetCurveKeys(CurveId, Curve.FloatCurve.GetConstRefOfKeys(), false);
						SetCurveColor(CurveId, Curve.GetColor(), false);
					}
				}
				
				for (const FTransformCurve& TransformCurve : InModel->GetCurveData().TransformCurves)
				{
					const FAnimationCurveIdentifier CurveId = Skeleton ? UAnimationCurveIdentifierExtensions::FindCurveIdentifier(Skeleton, TransformCurve.GetName(), ERawCurveTrackTypes::RCT_Transform) : FAnimationCurveIdentifier(TransformCurve.GetName(), ERawCurveTrackTypes::RCT_Transform);
					if (CurveId.IsValid() || IgnoreSkeletonValidation())
					{
						AddCurve(CurveId, TransformCurve.GetCurveTypeFlags());

						// Set each individual channel rich curve keys, to account for any custom tangents etc.
						for (int32 SubCurveIndex = 0; SubCurveIndex < 3; ++SubCurveIndex)
						{
							const ETransformCurveChannel Channel = static_cast<ETransformCurveChannel>(SubCurveIndex);
							const FVectorCurve* VectorCurve = TransformCurve.GetVectorCurveByIndex(SubCurveIndex);
							for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
							{
								const EVectorCurveChannel Axis = static_cast<EVectorCurveChannel>(ChannelIndex);
								FAnimationCurveIdentifier TargetCurveIdentifier = CurveId;
								UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(TargetCurveIdentifier, Channel, Axis);
								SetCurveKeys(TargetCurveIdentifier, VectorCurve->FloatCurves[ChannelIndex].GetConstRefOfKeys(), false);
							}
						}
					}
					else
					{
						ReportWarningf(LOCTEXT("InvalidCurveIdentifierFoundWarning", "Invalid transform curve identifier found: name: {0}, type: {1}"), FText::FromName(CurveId.CurveName), FText::AsNumber(static_cast<int32>(CurveId.CurveType)));
					}
				}
			}

			Model->AnimatedBoneAttributes = InModel->GetAttributes();
		}
		else
		{
			ReportError(LOCTEXT("UnableToGetOuterAnimSequenceError", "Unable to retrieve outer Animation Sequence"));
		}
	}
	else
	{
		ReportErrorf(LOCTEXT("InvalidExistingModel", "Invalid (existing) DataModel provided: {0} ({1})"), FText::FromName(InModel.GetObject() ? InModel.GetObject()->GetFName() : NAME_None),
			FText::FromName(InModel.GetObject() ? InModel.GetObject()->GetClass()->GetFName() : NAME_None));
	}
	CloseBracket();
}

#if WITH_EDITORONLY_DATA
void UAnimSequencerController::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UMovieScene::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UMovieSceneControlRigParameterTrack::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(UFKControlRig::StaticClass()));
}
#endif

void UAnimSequencerController::InitializeModel()
{
	if (const UAnimSequence* AnimSequence = Model->GetAnimationSequence())
	{
		if (Model->MovieScene == nullptr)
		{
			IAnimationDataModel::FEvaluationAndModificationLock Lock(*ModelInterface);
			if (UMovieScene* MovieScene = NewObject<UMovieScene>(Model.Get(), FName("MovieScene"), RF_Transactional))
			{
				Model->MovieScene = MovieScene;

				MovieScene->SetTickResolutionDirectly(UAnimationSettings::Get()->GetDefaultFrameRate());
				MovieScene->SetDisplayRate(UAnimationSettings::Get()->GetDefaultFrameRate());

				const TRange<FFrameNumber> DataRange = TRange<FFrameNumber>::Inclusive(FFrameNumber(0), 1);
				MovieScene->SetPlaybackRange(DataRange, false);

				if (AnimSequence->GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::IntroducingAnimationDataModel)
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					// Disabled for now as DDC had invalid data cached causing collapsed animations
					//Model->CachedRawDataGUID = AnimSequence->GenerateGuidFromRawData();
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				
				if(UMovieSceneControlRigParameterTrack* Track = MovieScene->AddTrack<UMovieSceneControlRigParameterTrack>())
				{
					if(UFKControlRig* ControlRig = NewObject<UFKControlRig>(Track, UFKControlRig::StaticClass(), NAME_None, RF_Transactional))
					{
						Model->InitializeFKControlRig(ControlRig, Model->GetSkeleton(), true);

						// Remove all control elements (start fresh)
						if (URigHierarchy* Hierarchy = ControlRig->GetHierarchy())
						{
							if (URigHierarchyController* Controller = Hierarchy->GetController())
							{
								TArray<FRigElementKey> ControlElementKeysToRemove;
								Hierarchy->ForEach<FRigControlElement>([&ControlElementKeysToRemove](const FRigControlElement* ControlElement) -> bool
								{
									ControlElementKeysToRemove.Add(ControlElement->GetKey());
									return true;
								});

								Hierarchy->ForEach<FRigCurveElement>([&ControlElementKeysToRemove](const FRigCurveElement* CurveElement) -> bool
								{
									ControlElementKeysToRemove.Add(CurveElement->GetKey());							
									return true;
								});

								for (const FRigElementKey& KeyToRemove : ControlElementKeysToRemove)
								{
									Controller->RemoveElement(KeyToRemove);
								}
							}
							else
							{
								ReportError(LOCTEXT("InvalidRigHierarchyController", "Non-valid RigHierarchyController found"));
							}

							ControlRig->RefreshActiveControls();
						}
						else
						{
							ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
						}

						UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(Track->CreateControlRigSection(0, ControlRig, true));				
						Section->SetRange(DataRange);
					}
					else
					{
						ReportError(LOCTEXT("FailedToCreateControlRig", "Failed to create new UFKControlRig"));
					}
				}
				else
				{
					ReportError(LOCTEXT("FailedToCreateTrack", "Failed to create UMovieSceneControlRigParameterTrack"));
				}
			}
			else
			{
				ReportError(LOCTEXT("FailedToCreateMovieScene", "Failed to create new UMovieScene"));
			}
		}
	}
	else
	{
		ReportError(LOCTEXT("UnableToGetOuterAnimSequenceError", "Unable to retrieve outer Animation Sequence"));
	}
}

bool UAnimSequencerController::AddBoneControl(const FName& BoneName) const
{
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		if (UFKControlRig* FKRig = Cast<UFKControlRig>(Section->GetControlRig()))
		{
			if(URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
			{
				const FRigElementKey BoneKey(URigHierarchy::GetSanitizedName(BoneName), ERigElementType::Bone);
				const FRigElementKey BoneNameControlKey(UFKControlRig::GetControlName(BoneKey.Name, ERigElementType::Bone), ERigElementType::Control);

				const bool bContainsBone = Hierarchy->Contains(BoneKey);
				const bool bContainsBoneControl = Hierarchy->Contains(BoneNameControlKey);

				if (bContainsBone && !bContainsBoneControl)
				{
					const FRigBoneElement* BoneElement = Hierarchy->FindChecked<FRigBoneElement>(BoneKey);
				
					const FRigElementKey ParentKey = Hierarchy->GetFirstParent(BoneKey);

					FRigControlSettings Settings;
					Settings.ControlType = ERigControlType::EulerTransform;
					Settings.DisplayName = BoneName;

					FTransform OffsetTransform;
					if (UAnimationSequencerDataModel::UseDirectFKControlRigMode == 0 && ParentKey.IsValid())
					{
						const FTransform GlobalTransform = Hierarchy->GetGlobalTransformByIndex(BoneElement->GetIndex(), true);
						const FTransform ParentTransform = Hierarchy->GetGlobalTransform(ParentKey, true);
						OffsetTransform = GlobalTransform.GetRelativeTransform(ParentTransform);
					}
					else
					{
						OffsetTransform = Hierarchy->GetLocalTransformByIndex(BoneElement->GetIndex(), true);
					}

					OffsetTransform.NormalizeRotation();

					if(URigHierarchyController* HierarchyController = Hierarchy->GetController())
					{
						HierarchyController->AddControl(BoneNameControlKey.Name, ParentKey, Settings, FRigControlValue::Make(FEulerTransform(OffsetTransform)), OffsetTransform, FTransform::Identity, false);

						if (FRigControlElement* BoneControlElement = Hierarchy->Find<FRigControlElement>(BoneNameControlKey))
						{
							Hierarchy->SetControlOffsetTransform(BoneControlElement, OffsetTransform, ERigTransformType::InitialLocal, false, false, true);					
							Hierarchy->SetControlOffsetTransform(BoneControlElement, OffsetTransform, ERigTransformType::CurrentLocal, false, false, true);

							Section->AddTransformParameter(BoneNameControlKey.Name, TOptional<FEulerTransform>(), false);

							FTransformParameterNameAndCurves* ParameterCurvePair = Section->GetTransformParameterNamesAndCurves().FindByPredicate([BoneNameControlKey](const FTransformParameterNameAndCurves& Parameter)
							{
								return Parameter.ParameterName == BoneNameControlKey.Name;
							});
							check(ParameterCurvePair);

							for (int32 Index = 0; Index < 3; ++Index)
							{
								ParameterCurvePair->Translation[Index].SetTickResolution(Model->GetFrameRate());
								ParameterCurvePair->Rotation[Index].SetTickResolution(Model->GetFrameRate());
								ParameterCurvePair->Scale[Index].SetTickResolution(Model->GetFrameRate());
							}
							
							return true;
						}
						else
						{
							ReportErrorf(LOCTEXT("FailedToAddBoneControlElement", "Failed to add Bone control with name {0}"), FText::FromName(BoneName));	
						}
					}
					else
					{
						ReportError(LOCTEXT("InvalidRigHierarchyController", "Non-valid RigHierarchyController found"));
					}
				}
				else
				{
					if (!bContainsBone)
					{
						ReportWarningf(LOCTEXT("FailedToFindHierarchyBone", "Unable to find RigBone with name {0}"), FText::FromName(BoneName));
					}

					if (bContainsBoneControl)
					{
						ReportWarningf(LOCTEXT("BoneControlElementAlreadyExists", "Failed to add Bone control with name {0} as it already exists"), FText::FromName(BoneNameControlKey.Name));			
					}
				}
			}
			else
			{
				ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
	}
	
	return false;
}

bool UAnimSequencerController::RemoveBoneControl(const FName& BoneName) const
{
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		if (UFKControlRig* FKRig = Cast<UFKControlRig>(Section->GetControlRig()))
		{
			if(URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
			{
				const FRigElementKey BoneNameControlKey(UFKControlRig::GetControlName(URigHierarchy::GetSanitizedName(BoneName), ERigElementType::Bone), ERigElementType::Control);
				
				const bool bContainsTransformCurve = Section->GetTransformParameterNamesAndCurves().ContainsByPredicate([BoneNameControlKey](const FTransformParameterNameAndCurves& Parameter)
				{
					return Parameter.ParameterName == BoneNameControlKey.Name;
				});
				
				const bool bContainsBoneControl = Hierarchy->Contains(BoneNameControlKey);
				if( bContainsBoneControl|| bContainsTransformCurve)
				{
					if(URigHierarchyController* HierarchyController = Hierarchy->GetController())
					{
						bool bRemoved = false;
						bRemoved |= (bContainsBoneControl && HierarchyController->RemoveElement(BoneNameControlKey));
						bRemoved |= (bContainsTransformCurve && Section->RemoveTransformParameter(BoneNameControlKey.Name));
						ensure(bRemoved);
				
						return true;
					}
					else
					{
						ReportError(LOCTEXT("InvalidRigHierarchyController", "Non-valid RigHierarchyController found"));
					}
				}
				else
				{
					ReportWarningf(LOCTEXT("FailedToFindHierarchyBoneControl", "Unable to find Bone control with name {0}"), FText::FromName(BoneNameControlKey.Name));
				}
			}
			else
			{
				ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
	}

	return false;
}

bool UAnimSequencerController::SetBoneCurveKeys(const FName& BoneName, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys) const
{
	check(UAnimationSequencerDataModel::UseDirectFKControlRigMode == 1);
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		UControlRig* ControlRig = Section->GetControlRig();
		if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
		{
			const URigHierarchy* Hierarchy = Model->GetControlRigHierarchy();
			if(Hierarchy || IgnoreSkeletonValidation())
			{
				const FRigElementKey BoneKey(URigHierarchy::GetSanitizedName(BoneName), ERigElementType::Bone);
				const FRigElementKey BoneNameControlKey(UFKControlRig::GetControlName(BoneKey.Name, ERigElementType::Bone), ERigElementType::Control);

				const bool bContainsBone = Hierarchy ? Hierarchy->Contains(BoneKey) : IgnoreSkeletonValidation();
				const bool bContainsBoneControl = Hierarchy ? Hierarchy->Contains(BoneNameControlKey) : IgnoreSkeletonValidation();

				if (bContainsBone && bContainsBoneControl)
				{
					Section->AddTransformParameter(BoneNameControlKey.Name, TOptional<FEulerTransform>(), false);
					FTransformParameterNameAndCurves* ParameterCurvePair = Section->GetTransformParameterNamesAndCurves().FindByPredicate([BoneNameControlKey](const FTransformParameterNameAndCurves& Parameter)
					{
						return Parameter.ParameterName == BoneNameControlKey.Name;
					});
					check(ParameterCurvePair);
					
					const int32 NumKeys = PositionalKeys.Num();

					// Check for whole range constant values
					const FVector3f& Translation0 = PositionalKeys[0];
					const FVector3f& EulerAngle0 = RotationalKeys[0].Euler();
					TArray<FVector3f> EulerAngles;
					EulerAngles.Reserve(RotationalKeys.Num());
					EulerAngles.Add(EulerAngle0);
					const FVector3f& Scale0 = ScalingKeys[0];
					TArray<bool> AreKeysVarying;
					AreKeysVarying.SetNumZeroed(9);
					{
						for (int32 KeyIndex = 1; KeyIndex < NumKeys; ++KeyIndex)
						{
							EulerAngles.Add(RotationalKeys[KeyIndex].Euler());
							for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
							{
								AreKeysVarying[ChannelIndex] |= !FMath::IsNearlyEqual(Translation0[ChannelIndex], PositionalKeys[KeyIndex][ChannelIndex], UE_KINDA_SMALL_NUMBER);
								AreKeysVarying[3 + ChannelIndex] |= !FMath::IsNearlyEqual(EulerAngle0[ChannelIndex], EulerAngles.Last()[ChannelIndex], UE_KINDA_SMALL_NUMBER);
								AreKeysVarying[6 + ChannelIndex] |= !FMath::IsNearlyEqual(Scale0[ChannelIndex], ScalingKeys[KeyIndex][ChannelIndex], UE_KINDA_SMALL_NUMBER);
							}

							if (!AreKeysVarying.Contains(false))
							{
								break;
							}
						}
					}

					const bool bAllKeysAreConstant = !AreKeysVarying.Contains(true);
					const int32 MaximumNumberOfKeys = bAllKeysAreConstant ? 1 : NumKeys;

					struct FScratchMemory : public TThreadSingleton<FScratchMemory>
					{
						TArray<FFrameNumber> FrameNumbers;
						TArray<FMovieSceneFloatValue> Values[9];
					};
					
					FScratchMemory& ScratchMemory = FScratchMemory::Get();
					ScratchMemory.FrameNumbers.Reset(MaximumNumberOfKeys);
					for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
					{
						if (!AreKeysVarying[ChannelIndex])
						{
							ScratchMemory.Values[ChannelIndex].Reset(1);
							FMovieSceneFloatValue& Value = ScratchMemory.Values[ChannelIndex].AddDefaulted_GetRef();
							Value.Value = Translation0[ChannelIndex];
							Value.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						}
						else
						{
							ScratchMemory.Values[ChannelIndex].SetNum(MaximumNumberOfKeys, EAllowShrinking::No);
						}
						
						if (!AreKeysVarying[3 + ChannelIndex])
						{
							ScratchMemory.Values[3 + ChannelIndex].Reset(1);
							FMovieSceneFloatValue& Value = ScratchMemory.Values[3 + ChannelIndex].AddDefaulted_GetRef();
							Value.Value = EulerAngle0[ChannelIndex];
							Value.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						}
						else
						{
							ScratchMemory.Values[3 + ChannelIndex].SetNum(MaximumNumberOfKeys, EAllowShrinking::No);
						}
						
						if (!AreKeysVarying[6 + ChannelIndex])
						{
							ScratchMemory.Values[6 + ChannelIndex].Reset(1);
							FMovieSceneFloatValue& Value = ScratchMemory.Values[6 + ChannelIndex].AddDefaulted_GetRef();
							Value.Value = Scale0[ChannelIndex];
							Value.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						}
						else
						{
							ScratchMemory.Values[6 + ChannelIndex].SetNum(MaximumNumberOfKeys, EAllowShrinking::No);
						}

						// Reset all curve data
						ParameterCurvePair->Translation[ChannelIndex].Reset();
						ParameterCurvePair->Rotation[ChannelIndex].Reset();
						ParameterCurvePair->Scale[ChannelIndex].Reset(); 
					}
					
					if (MaximumNumberOfKeys > 1)
					{
						for (int32 KeyIndex = 0; KeyIndex < MaximumNumberOfKeys; ++KeyIndex)
						{
							ScratchMemory.FrameNumbers.Add(KeyIndex);
						
							// Translation
							const auto& Location = PositionalKeys[KeyIndex];						
							// Rotation (try and retrieve cached version)			
							const auto& EulerRotation = EulerAngles.IsValidIndex(KeyIndex) ? EulerAngles[KeyIndex] : RotationalKeys[KeyIndex].Euler();
							// Scaling
							const auto& Scale = ScalingKeys[KeyIndex];
						
							for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
							{
								if (AreKeysVarying[ChannelIndex])
								{
									// Pos
									FMovieSceneFloatValue& Value = ScratchMemory.Values[ChannelIndex][KeyIndex];
									Value.Value = Location[ChannelIndex];
									Value.InterpMode = ERichCurveInterpMode::RCIM_Linear;
								}

								if (AreKeysVarying[3 + ChannelIndex])
								{
									// Rot
									FMovieSceneFloatValue& Value = ScratchMemory.Values[3 + ChannelIndex][KeyIndex];
									Value.Value = EulerRotation[ChannelIndex];
									Value.InterpMode = ERichCurveInterpMode::RCIM_Linear;
								}

								if (AreKeysVarying[6 + ChannelIndex])
								{
									// Scale
									FMovieSceneFloatValue& Value = ScratchMemory.Values[6 + ChannelIndex][KeyIndex];					
									Value.Value = Scale[ChannelIndex];
									Value.InterpMode = ERichCurveInterpMode::RCIM_Linear;
								}
							}
						}
					}

					TArray<FFrameNumber> ConstantFrameNumber = { 0 };
					for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
					{
						if (AreKeysVarying[ChannelIndex])
						{
							ParameterCurvePair->Translation[ChannelIndex].SetKeysOnly(AreKeysVarying[ChannelIndex] ? ScratchMemory.FrameNumbers : ConstantFrameNumber, ScratchMemory.Values[ChannelIndex]);
						}
						else
						{
							ParameterCurvePair->Translation[ChannelIndex].SetDefault(ScratchMemory.Values[ChannelIndex][0].Value);
						}

						if (AreKeysVarying[3 + ChannelIndex])
						{
							ParameterCurvePair->Rotation[ChannelIndex].SetKeysOnly(AreKeysVarying[3 + ChannelIndex] ? ScratchMemory.FrameNumbers : ConstantFrameNumber, ScratchMemory.Values[3 + ChannelIndex]);
						}
						else
						{
							ParameterCurvePair->Rotation[ChannelIndex].SetDefault(ScratchMemory.Values[3 + ChannelIndex][0].Value);
						}

						if (AreKeysVarying[6 + ChannelIndex])
						{
							ParameterCurvePair->Scale[ChannelIndex].SetKeysOnly(AreKeysVarying[6 + ChannelIndex] ? ScratchMemory.FrameNumbers : ConstantFrameNumber,  ScratchMemory.Values[6 + ChannelIndex]);
						}
						else
						{
							ParameterCurvePair->Scale[ChannelIndex].SetDefault(ScratchMemory.Values[6 + ChannelIndex][0].Value);
						}
					}
					
					return true;
					
				}
			}
		}
	}

	return false;
}

bool UAnimSequencerController::UpdateBoneCurveKeys(const FName& BoneName, const TArray<FTransform>& Keys, const TArray<float>& TimeValues) const
{
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{		
		UControlRig* ControlRig = Section->GetControlRig();
		if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
		{
			if(URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
			{
				const FRigElementKey BoneKey(URigHierarchy::GetSanitizedName(BoneName), ERigElementType::Bone);
				const FRigElementKey BoneNameControlKey(UFKControlRig::GetControlName(BoneKey.Name, ERigElementType::Bone), ERigElementType::Control);

				const bool bContainsBone = Hierarchy->Contains(BoneKey);
				const bool bContainsBoneControl = Hierarchy->Contains(BoneNameControlKey);
				const bool bContainsParameter = Section->HasTransformParameter(BoneNameControlKey.Name);

				const bool bIsSorted = Algo::IsSorted(TimeValues);
				if (bContainsBone && bContainsBoneControl && bContainsParameter && bIsSorted)
				{
					FTransformParameterNameAndCurves* ParameterCurvePair = Section->GetTransformParameterNamesAndCurves().FindByPredicate([BoneNameControlKey](const FTransformParameterNameAndCurves& Parameter)
					{
						return Parameter.ParameterName == BoneNameControlKey.Name;
					});
					check(ParameterCurvePair);

					const FTransform Offset = Hierarchy->GetControlOffsetTransform(Hierarchy->FindChecked<FRigControlElement>(BoneNameControlKey), ERigTransformType::InitialLocal);
					const int32 NumKeys = Keys.Num();
					TArray<FFrameNumber> InTimes;
					TArray<FMovieSceneFloatValue> LocationX;
					TArray<FMovieSceneFloatValue> LocationY;
					TArray<FMovieSceneFloatValue> LocationZ;
					TArray<FMovieSceneFloatValue> EulerX;
					TArray<FMovieSceneFloatValue> EulerY;
					TArray<FMovieSceneFloatValue> EulerZ;
					TArray<FMovieSceneFloatValue> ScaleX;
					TArray<FMovieSceneFloatValue> ScaleY;
					TArray<FMovieSceneFloatValue> ScaleZ;
					InTimes.Reserve(NumKeys);
					LocationX.Reserve(NumKeys);
					LocationY.Reserve(NumKeys);
					LocationZ.Reserve(NumKeys);
					EulerX.Reserve(NumKeys);
					EulerY.Reserve(NumKeys);
					EulerZ.Reserve(NumKeys);
					ScaleX.Reserve(NumKeys);
					ScaleY.Reserve(NumKeys);
					ScaleZ.Reserve(NumKeys);

					for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
					{
						const FTransform& Transform = Keys[KeyIndex];

						FTransform OffsetTransform = UAnimationSequencerDataModel::UseDirectFKControlRigMode ? Transform : Transform.GetRelativeTransform(Offset);
						OffsetTransform.NormalizeRotation();

						const FFrameNumber& FrameNumber = Model->MovieScene->GetTickResolution().AsFrameTime(TimeValues[KeyIndex]).RoundToFrame();
						const FVector& Location = OffsetTransform.GetLocation();
						const FVector& EulerAngles = OffsetTransform.GetRotation().Euler();
						const FVector& Scale = OffsetTransform.GetScale3D();

						FMovieSceneFloatValue LocationValueX(Location.X);
						FMovieSceneFloatValue LocationValueY(Location.Y);
						FMovieSceneFloatValue LocationValueZ(Location.Z);
						LocationValueX.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						LocationValueY.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						LocationValueZ.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						FMovieSceneFloatValue EulerValueX(EulerAngles.X);
						FMovieSceneFloatValue EulerValueY(EulerAngles.Y);
						FMovieSceneFloatValue EulerValueZ(EulerAngles.Z);
						EulerValueX.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						EulerValueY.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						EulerValueZ.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						FMovieSceneFloatValue ScaleValueX(Scale.X);
						FMovieSceneFloatValue ScaleValueY(Scale.Y);
						FMovieSceneFloatValue ScaleValueZ(Scale.Z);
						ScaleValueX.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						ScaleValueY.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						ScaleValueZ.InterpMode = ERichCurveInterpMode::RCIM_Linear;
						InTimes.Add(FrameNumber);
						LocationX.Add(LocationValueX);
						LocationY.Add(LocationValueY);
						LocationZ.Add(LocationValueZ);
						EulerX.Add(EulerValueX);
						EulerY.Add(EulerValueY);
						EulerZ.Add(EulerValueZ);
						ScaleX.Add(ScaleValueX);
						ScaleY.Add(ScaleValueY);
						ScaleZ.Add(ScaleValueZ);
					}

					ParameterCurvePair->Translation[0].UpdateOrAddKeys(InTimes, LocationX);
					ParameterCurvePair->Translation[1].UpdateOrAddKeys(InTimes, LocationY);
					ParameterCurvePair->Translation[2].UpdateOrAddKeys(InTimes, LocationZ);
					ParameterCurvePair->Rotation[0].UpdateOrAddKeys(InTimes, EulerX);
					ParameterCurvePair->Rotation[1].UpdateOrAddKeys(InTimes, EulerY);
					ParameterCurvePair->Rotation[2].UpdateOrAddKeys(InTimes, EulerZ);
					ParameterCurvePair->Scale[0].UpdateOrAddKeys(InTimes, ScaleX);
					ParameterCurvePair->Scale[1].UpdateOrAddKeys(InTimes, ScaleY);
					ParameterCurvePair->Scale[2].UpdateOrAddKeys(InTimes, ScaleZ);
					ParameterCurvePair->Translation[0].ClearDefault();
					ParameterCurvePair->Translation[1].ClearDefault();
					ParameterCurvePair->Translation[2].ClearDefault();
					ParameterCurvePair->Rotation[0].ClearDefault();
					ParameterCurvePair->Rotation[1].ClearDefault();
					ParameterCurvePair->Rotation[2].ClearDefault();
					ParameterCurvePair->Scale[0].ClearDefault();
					ParameterCurvePair->Scale[1].ClearDefault();
					ParameterCurvePair->Scale[2].ClearDefault();
					return true;
				}
				else
				{
					if (!bContainsBone)
					{
						ReportWarningf(LOCTEXT("FailedToFindHierarchyBone", "Unable to find RigBone with name {0}"), FText::FromName(BoneName));
					}
					if (!bContainsBoneControl)
					{
						ReportWarningf(LOCTEXT("BoneControlElementNotFound", "Failed to find Bone control with name {0}"), FText::FromName(BoneNameControlKey.Name));
					}

					if (!bContainsParameter)
					{
						ReportWarningf(LOCTEXT("BoneControlParameterNotFound", "Failed to find Bone control curve with name {0}"), FText::FromName(BoneNameControlKey.Name));
					}
					if (!bIsSorted)
					{
						ReportWarning(LOCTEXT("TimeValuesNotSorted", "Given Time Values are not sorted"));
					}
				}
			}
			else
			{
				ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
	}

	return false;
}

bool UAnimSequencerController::RemoveBoneCurveKey(const FName& BoneName, float Time)
{
	if (const UAnimSequence* AnimSequence = ModelInterface->GetAnimationSequence())
    {
    	USkeleton* Skeleton = AnimSequence->GetSkeleton();
    
    	const FAnimationCurveIdentifier CurveId = UAnimationCurveIdentifierExtensions::GetCurveIdentifier(Skeleton, BoneName, ERawCurveTrackTypes::RCT_Transform);
		if (const UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
		{
			UControlRig* ControlRig = Section->GetControlRig();
	
			if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
			{
				if(const URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
				{
					const FRigElementKey BoneKey(URigHierarchy::GetSanitizedName(BoneName), ERigElementType::Bone);
					const FRigElementKey BoneNameControlKey(UFKControlRig::GetControlName(BoneKey.Name, ERigElementType::Bone), ERigElementType::Control);
	
					const bool bContainsBone = Hierarchy->Contains(BoneKey);
					const bool bContainsBoneControl = Hierarchy->Contains(BoneNameControlKey);
	
					if (bContainsBone && bContainsBoneControl)
					{
						if(Section->HasTransformParameter(BoneNameControlKey.Name))
						{
							bool bResult = false;
							for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
							{
								FAnimationCurveIdentifier SubCurve = CurveId;
								
								UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(SubCurve, ETransformCurveChannel::Position, static_cast<EVectorCurveChannel>(ChannelIndex));
								bResult |= RemoveCurveKey(SubCurve, Time);
	
								UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(SubCurve, ETransformCurveChannel::Rotation, static_cast<EVectorCurveChannel>(ChannelIndex));
								bResult |= RemoveCurveKey(SubCurve, Time);
	
								UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(SubCurve, ETransformCurveChannel::Scale, static_cast<EVectorCurveChannel>(ChannelIndex));
								bResult |= RemoveCurveKey(SubCurve, Time);
							}
							
							return bResult;
						}
						else
						{							
							ReportWarningf(LOCTEXT("BoneControlParameterNotFound", "Failed to find Bone control curve with name {0}"), FText::FromName(BoneNameControlKey.Name));
						}
					}
					else
					{
						if (!bContainsBone)
						{
							ReportWarningf(LOCTEXT("FailedToFindHierarchyBone", "Unable to find RigBone with name {0}"), FText::FromName(BoneName));						
						}

						if (!bContainsBoneControl)
						{
							ReportWarningf(LOCTEXT("BoneControlElementNotFound", "Failed to find Bone control with name {0}"), FText::FromName(BoneNameControlKey.Name));			
						}
					}
				}
				else
				{
					ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
				}
			}
			else
			{
				ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
		}
	}
	
	return false;
}

bool UAnimSequencerController::AddCurveControl(const FName& CurveName) const
{
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		UControlRig* ControlRig = Section->GetControlRig();
		if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
		{
			if(URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
			{				
				const FRigElementKey CurveKey(URigHierarchy::GetSanitizedName(CurveName), ERigElementType::Curve);
				const FRigElementKey CurveControlKey(UFKControlRig::GetControlName(CurveKey.Name, ERigElementType::Curve), ERigElementType::Control);

				const bool bContainsCurve = Hierarchy->Contains(CurveKey);
				const bool bContainsCurveControl = Hierarchy->Contains(CurveControlKey);

				if(URigHierarchyController* HierarchyController = Hierarchy->GetController())
				{
					if(!bContainsCurve)
					{
						HierarchyController->AddCurve(CurveKey.Name, 0.f, false);		
					}

					const bool bHasCurveControlChannel = Section->HasScalarParameter(CurveControlKey.Name);
					if (!bHasCurveControlChannel)
					{
						Section->AddScalarParameter(CurveControlKey.Name, TOptional<float>(), false);
					}

					FScalarParameterNameAndCurve* ParameterCurvePair = Section->GetScalarParameterNamesAndCurves().FindByPredicate([CurveControlKey](const FScalarParameterNameAndCurve& Parameter)
					{
						return Parameter.ParameterName == CurveControlKey.Name;
					});
					check(ParameterCurvePair);

					if(!bHasCurveControlChannel)
					{
						ParameterCurvePair->ParameterCurve.SetTickResolution(Model->GetFrameRate());
					}
				
					const FRigCurveElement* CurveElement = Hierarchy->FindChecked<FRigCurveElement>(CurveKey);
					ensure(CurveElement);
					if (!bContainsCurveControl && CurveElement)
					{
						FRigControlSettings Settings;
						Settings.ControlType = ERigControlType::Float;
						Settings.DisplayName = FName(*(CurveKey.Name.ToString() + TEXT(" Curve")));
								
						HierarchyController->AddControl(CurveControlKey.Name, FRigElementKey(), Settings, FRigControlValue::Make(CurveElement->Get()), FTransform::Identity, FTransform::Identity, false);

						const FRigControlElement* ControlElement = Hierarchy->FindChecked<FRigControlElement>(CurveControlKey);
						ensure(ControlElement);
						
						return true;
					}
					else
					{					
						ReportWarningf(LOCTEXT("FailedToAddCurvecontrol", "Failed to add Curve control with name {0} as it already exists"), FText::FromName(CurveControlKey.Name));			
					}
				}
				else
				{
					ReportError(LOCTEXT("InvalidRigHierarchyController", "Non-valid RigHierarchyController found"));
				}
			}
			else
			{
				ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
	}

	return false;
}

bool UAnimSequencerController::RenameCurveControl(const FName& CurveName, const FName& NewCurveName) const
{
	// Need to rename curve and control element and curve				
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		if (UControlRig* ControlRig = Section->GetControlRig())
		{
			if (URigHierarchy* RigHierarchy = Model->GetControlRigHierarchy())
			{
				if (URigHierarchyController* Controller = RigHierarchy->GetController())
				{
					const FRigElementKey CurveKey(URigHierarchy::GetSanitizedName(CurveName), ERigElementType::Curve);
					const FRigElementKey CurveControlKey(UFKControlRig::GetControlName(CurveKey.Name, ERigElementType::Curve), ERigElementType::Control);

					const FName SanitizedNewCurveName = URigHierarchy::GetSanitizedName(NewCurveName);
					// Rename the curve element itself
					if (RigHierarchy->Contains(CurveKey))
					{											
						const FRigElementKey NewCurveKey = Controller->RenameElement(CurveKey, SanitizedNewCurveName, false);
						if (NewCurveKey.IsValid())
						{
							// Rename the control for the curve value
							if (RigHierarchy->Contains(CurveControlKey))
							{
								const FRigElementKey NewCurveControlKey = Controller->RenameElement(CurveControlKey, UFKControlRig::GetControlName(NewCurveKey.Name, ERigElementType::Curve), false);
								if (NewCurveControlKey.IsValid())
								{
									FRigControlElement* ControlElement = RigHierarchy->Find<FRigControlElement>(NewCurveControlKey);
									ControlElement->Settings.DisplayName = FName(*(NewCurveKey.Name.ToString() + TEXT(" Curve")));

									// Rename the curve driving the control value
									FScalarParameterNameAndCurve* ParameterCurvePair = Section->GetScalarParameterNamesAndCurves().FindByPredicate([CurveControlKey](const FScalarParameterNameAndCurve& Parameter)
									{
										return Parameter.ParameterName == CurveControlKey.Name;
									});
					
									if(ParameterCurvePair)
									{
										ParameterCurvePair->ParameterName = NewCurveControlKey.Name;
									}

									return true;
								}
								else
								{
									ReportErrorf(LOCTEXT("FailedToRenameCurveControl", "Failed to rename RigCurve with name {0}"), FText::FromName(CurveControlKey.Name));						
								}
							}
							else
							{
								ReportErrorf(LOCTEXT("UnableToRenameCurveControl", "Unable to rename Curve control with name {0}"), FText::FromName(CurveControlKey.Name));							
							}
						}
						else
						{
							ReportErrorf(LOCTEXT("FailedToRenameCurve", "Failed to rename RigCurve with name {0}"), FText::FromName(NewCurveName));						
						}
					}
					else
					{
						ReportErrorf(LOCTEXT("UnableToRenameCurve", "Unable to rename RigCurve with name {0}"), FText::FromName(NewCurveName));						
					}

					return false;
				}
				else
				{
					ReportError(LOCTEXT("InvalidRigHierarchyController", "Non-valid RigHierarchyController found"));
				}
			}
			else
			{
				ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
	}

	return false;
}

bool UAnimSequencerController::RemoveCurveControl(const FName& CurveName) const
{
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		UControlRig* ControlRig = Section->GetControlRig();

		if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
		{
			if(URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
			{									
				const FRigElementKey CurveKey(URigHierarchy::GetSanitizedName(CurveName), ERigElementType::Curve);
				const FRigElementKey CurveControlKey(UFKControlRig::GetControlName(CurveKey.Name, ERigElementType::Curve), ERigElementType::Control);

				const bool bContainsCurve = Hierarchy->Contains(CurveKey);
				const bool bContainsCurveControl = Hierarchy->Contains(CurveControlKey);

				if(URigHierarchyController* HierarchyController = Hierarchy->GetController())
				{
					if(bContainsCurve)
					{
						HierarchyController->RemoveElement(CurveKey);

						if (bContainsCurveControl)
						{
							HierarchyController->RemoveElement(CurveControlKey);
							Section->RemoveScalarParameter(CurveControlKey.Name);
    
							return true;
						}
						else
						{
							ReportErrorf(LOCTEXT("FailedtoFindCurveControl", "Failed to find Curve control with name {0}"), FText::FromName(CurveControlKey.Name));			
						}

					
					}
					else
					{
						ReportErrorf(LOCTEXT("FailedtoFindRigCurve", "Failed to find RigCurve with name {0}"), FText::FromName(CurveName));			
					}
				}
				else
				{
					ReportError(LOCTEXT("InvalidRigHierarchyController", "Non-valid RigHierarchyController found"));
				}				
			}
			else
			{
				ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
	}
	
	return false;
}

bool UAnimSequencerController::SetCurveControlKeys(const FName& CurveName, const TArray<FRichCurveKey>& CurveKeys) const
{
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		UControlRig* ControlRig = Section->GetControlRig();

		if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
		{
			const URigHierarchy* Hierarchy = Model->GetControlRigHierarchy();

			if (Hierarchy ||  IgnoreSkeletonValidation())
			{
				const FRigElementKey CurveKey(URigHierarchy::GetSanitizedName(CurveName), ERigElementType::Curve);
				const FRigElementKey CurveControlKey(UFKControlRig::GetControlName(CurveKey.Name, ERigElementType::Curve), ERigElementType::Control);

				const bool bContainsCurve = Hierarchy ? Hierarchy->Contains(CurveKey) : IgnoreSkeletonValidation();
				const bool bContainsCurveControl = Hierarchy ? Hierarchy->Contains(CurveControlKey) : IgnoreSkeletonValidation();

				if(bContainsCurve && bContainsCurveControl)
				{
					const bool bHasCurveControlChannel = Section->HasScalarParameter(CurveControlKey.Name);
					if (!bHasCurveControlChannel)
					{
						Section->AddScalarParameter(CurveControlKey.Name, TOptional<float>(), Model->bPopulated);
					}

					FScalarParameterNameAndCurve* ParameterCurvePair = Section->GetScalarParameterNamesAndCurves().FindByPredicate([CurveControlKey](const FScalarParameterNameAndCurve& Parameter)
					{
						return Parameter.ParameterName == CurveControlKey.Name;
					});
					check(ParameterCurvePair);

					if(!bHasCurveControlChannel)
					{
						ParameterCurvePair->ParameterCurve.SetTickResolution(Model->GetFrameRate());
					}
					
					/*
					 * Scenarios for curve keys conversion
					 *
					 * 1. All keys align with frame boundaries -> add all and convert between key types
					 * 2. Some keys do not align with frame boundaries -> resample keys to MaxChannelRate (240000fps) to minimize evaluated value deltas
					 */
					const FFrameRate& ModelRate = Model->GetFrameRate();
					const bool bContainsOffFrameCurveKeys = CurveKeys.ContainsByPredicate([ModelRate](const FRichCurveKey& Key)
					{
						const FFrameTime FrameTime = ModelRate.AsFrameTime(Key.Time);
						return !(FMath::IsNearlyZero(FrameTime.GetSubFrame(), KINDA_SMALL_NUMBER) || FMath::IsNearlyEqual(FrameTime.GetSubFrame(), 1.0f, KINDA_SMALL_NUMBER));
					});
					
					const FFrameRate MaxChannelRate = FFrameRate(240000, 1);
					const FFrameRate SamplingRate = bContainsOffFrameCurveKeys ? MaxChannelRate : ModelRate;
					ParameterCurvePair->ParameterCurve.SetTickResolution(SamplingRate);
					AnimSequencerHelpers::ConvertRichCurveKeysToFloatChannel(CurveKeys, ParameterCurvePair->ParameterCurve);

					return true;					
				}
				else
				{
					if(!bContainsCurveControl)
					{
						ReportErrorf(LOCTEXT("FailedtoFindCurveControl", "Failed to find Curve control with name {0}"), FText::FromName(CurveControlKey.Name));			
					}

					if(!bContainsCurve)
					{
						ReportErrorf(LOCTEXT("FailedtoFindRigCurve", "Failed to find RigCurve with name {0}"), FText::FromName(CurveName));			
					}
				}
			}
			else
			{
				ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
	}

	return false;
}

bool UAnimSequencerController::SetCurveControlKey(const FName& CurveName, const FRichCurveKey& Key, bool bUpdateKey, float& OutKeyTiming) const
{
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		UControlRig* ControlRig = Section->GetControlRig();

		if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
		{
			if(const URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
			{
				const FRigElementKey CurveKey(URigHierarchy::GetSanitizedName(CurveName), ERigElementType::Curve);
				const FRigElementKey CurveControlKey(UFKControlRig::GetControlName(CurveKey.Name, ERigElementType::Curve), ERigElementType::Control);
				const bool bContainsCurve = Hierarchy->Contains(CurveKey);
				const bool bContainsCurveControl = Hierarchy->Contains(CurveControlKey);
				
				if (bContainsCurve && bContainsCurveControl)
				{
					const bool bHasCurveControlChannel = Section->HasScalarParameter(CurveControlKey.Name);
					if(!bHasCurveControlChannel)
					{
						Section->AddScalarParameter(CurveControlKey.Name, TOptional<float>(), Model->bPopulated);
					}

					FScalarParameterNameAndCurve* ParameterCurvePair = Section->GetScalarParameterNamesAndCurves().FindByPredicate([CurveControlKey](const FScalarParameterNameAndCurve& Parameter)
					{
						return Parameter.ParameterName == CurveControlKey.Name;
					});
					check(ParameterCurvePair);
					
					if(!bHasCurveControlChannel)
					{
						ParameterCurvePair->ParameterCurve.SetTickResolution(Model->GetFrameRate());
					}
					
					FMovieSceneFloatValue MovieSceneValue;
					AnimSequencerHelpers::ConvertRichCurveKeyToFloatValue(Key, MovieSceneValue );

					/*
					 * Scenarios for curve keys conversion
					 *
					 * 1. All keys align with frame boundaries -> add all and convert between key types
					 * 2. Some keys do not align with frame boundaries -> resample keys to MaxChannelRate (240000fps) to minimize evaluated value deltas
					 */
					const FFrameRate CurveRate = ParameterCurvePair->ParameterCurve.GetTickResolution();
					const FFrameTime CurveFrameTime = CurveRate.AsFrameTime(Key.Time);
					const bool bOffFrameCurveKey = !(FMath::IsNearlyZero(CurveFrameTime.GetSubFrame(), KINDA_SMALL_NUMBER) || FMath::IsNearlyEqual(CurveFrameTime.GetSubFrame(), 1.0f, KINDA_SMALL_NUMBER));
					if(bOffFrameCurveKey)
					{
						const FFrameRate MaxChannelRate = FFrameRate(240000, 1);
						ParameterCurvePair->ParameterCurve.RemapTimes(UE::MovieScene::FFrameRateRetiming{ CurveRate, MaxChannelRate });
						ParameterCurvePair->ParameterCurve.SetTickResolution(MaxChannelRate);
					}					

					const FFrameNumber FrameNumber = ParameterCurvePair->ParameterCurve.GetTickResolution().AsFrameTime(Key.Time).RoundToFrame();
					check(bUpdateKey || ParameterCurvePair->ParameterCurve.GetData().FindKey(FrameNumber) == INDEX_NONE);
					const FKeyHandle KeyHandle = ParameterCurvePair->ParameterCurve.GetData().UpdateOrAddKey(FrameNumber, MovieSceneValue);
					
					OutKeyTiming = ParameterCurvePair->ParameterCurve.GetTickResolution().AsSeconds(FrameNumber);
					return KeyHandle != FKeyHandle::Invalid();
				}
				else
				{
					if(!bContainsCurveControl)
					{
						ReportErrorf(LOCTEXT("FailedtoFindCurveControl", "Failed to find Curve control with name {0}"), FText::FromName(CurveControlKey.Name));			
					}

					if(!bContainsCurve)
					{
						ReportErrorf(LOCTEXT("FailedtoFindRigCurve", "Failed to find RigCurve with name {0}"), FText::FromName(CurveName));			
					}
				}
			}
			else
			{
				ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
	}

	return false;
}

bool UAnimSequencerController::RemoveCurveControlKey(const FName& CurveName, float Time) const
{
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		UControlRig* ControlRig = Section->GetControlRig();

		if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
		{
			if(const URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
			{
				const FFrameNumber FrameNumber = Model->MovieScene->GetTickResolution().AsFrameNumber(Time);
				const FRigElementKey CurveKey(URigHierarchy::GetSanitizedName(CurveName), ERigElementType::Curve);
				const FRigElementKey CurveControlKey(UFKControlRig::GetControlName(CurveKey.Name, ERigElementType::Curve), ERigElementType::Control);
				const bool bContainsCurve = Hierarchy->Contains(CurveKey);
				const bool bContainsCurveControl = Hierarchy->Contains(CurveControlKey);

				if (bContainsCurve && bContainsCurveControl)
				{
					if(Section->HasScalarParameter(CurveControlKey.Name))
					{
						FScalarParameterNameAndCurve* ParameterCurvePair = Section->GetScalarParameterNamesAndCurves().FindByPredicate([CurveControlKey](const FScalarParameterNameAndCurve& Parameter)
						{
							return Parameter.ParameterName == CurveControlKey.Name;
						});
						check(ParameterCurvePair);

						const int32 Index = ParameterCurvePair->ParameterCurve.GetData().FindKey(FrameNumber);
						if (Index != INDEX_NONE)
						{
							ParameterCurvePair->ParameterCurve.GetData().RemoveKey(Index);
							
							return true;
						}
					}
				}
				else
				{
					if(!bContainsCurveControl)
					{
						ReportErrorf(LOCTEXT("FailedtoFindCurveControl", "Failed to find Curve control with name {0}"), FText::FromName(CurveControlKey.Name));			
					}

					if(!bContainsCurve)
					{
						ReportErrorf(LOCTEXT("FailedtoFindRigCurve", "Failed to find RigCurve with name {0}"), FText::FromName(CurveName));			
					}
				}
			}
			else
			{
				ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
	}
	
	return false;
}

bool UAnimSequencerController::DuplicateCurveControl(const FName& CurveName, const FName& DuplicateCurveName) const
{
	if (UMovieSceneControlRigParameterSection* Section = Model->GetFKControlRigSection())
	{
		if (UFKControlRig* FKRig = Cast<UFKControlRig>(Section->GetControlRig()))
		{
			if (URigHierarchy* Hierarchy = Model->GetControlRigHierarchy())
			{
				if (URigHierarchyController* HierarchyController = Hierarchy->GetController())
				{
					const FRigElementKey NewCurveKey(URigHierarchy::GetSanitizedName(DuplicateCurveName), ERigElementType::Curve);
					const FRigElementKey NewCurveControlKey(UFKControlRig::GetControlName(NewCurveKey.Name, ERigElementType::Curve), ERigElementType::Control);

					const bool bAlreadyContainsCurve = Hierarchy->Contains(NewCurveKey);
					const bool bAlreadyContainsCurveControl = Hierarchy->Contains(NewCurveControlKey);

					if(!bAlreadyContainsCurve && !bAlreadyContainsCurveControl)
					{
						HierarchyController->AddCurve(NewCurveKey.Name, 0.f, false);
						const FRigCurveElement* CurveElement = Hierarchy->FindChecked<FRigCurveElement>(NewCurveKey);

						FRigControlSettings Settings;
						Settings.ControlType = ERigControlType::Float;
						Settings.DisplayName = FName(*(NewCurveKey.Name.ToString() + TEXT(" Curve")));
			
						HierarchyController->AddControl(NewCurveControlKey.Name, FRigElementKey(), Settings, FRigControlValue::Make(CurveElement->Get()), FTransform::Identity, FTransform::Identity, false);
			
						// Rename the curve driving the control value
						const FScalarParameterNameAndCurve* ParameterCurvePair = Section->GetScalarParameterNamesAndCurves().FindByPredicate([CurveName](const FScalarParameterNameAndCurve& Parameter)
						{
							return Parameter.ParameterName == UFKControlRig::GetControlName(URigHierarchy::GetSanitizedName(CurveName), ERigElementType::Curve);
						});
				
						if(ParameterCurvePair)
						{
							Section->AddScalarParameter(NewCurveControlKey.Name, TOptional<float>(), true);

							FScalarParameterNameAndCurve* DuplicatedParameterCurvePair = Section->GetScalarParameterNamesAndCurves().FindByPredicate([NewCurveControlKey](const FScalarParameterNameAndCurve& Parameter)
							{
								return Parameter.ParameterName == NewCurveControlKey.Name;
							});

							// This is suppose to have been added
							check(DuplicatedParameterCurvePair);

							// Copy over curve data
							DuplicatedParameterCurvePair->ParameterCurve = ParameterCurvePair->ParameterCurve;
						}

						return true;						
					}
					else
					{
						if(bAlreadyContainsCurveControl)
						{
							ReportErrorf(LOCTEXT("CurveControlAlreadyExists", "Curve control with name {0} already exists"), FText::FromName(NewCurveControlKey.Name));			
						}

						if(bAlreadyContainsCurve)
						{
							ReportErrorf(LOCTEXT("RigCurveAlreadyExists", "RigCurve with name {0} already exists"), FText::FromName(DuplicateCurveName));			
						}
					}

					return true;
				}
			}
			else
			{
				ReportError(LOCTEXT("InvalidRigHierarchy", "Unable to retrieve valid URigHierarchy"));
			}
		}
		else
		{
			ReportError(LOCTEXT("InvalidControlRig", "Unable to retrieve valid UFKControlRig"));
		}
	}
	else
	{
		ReportError(LOCTEXT("InvalidControlRigSection", "Unable to retrieve valid UMovieSceneControlRigParameterSection"));
	}
	return false;
}

#undef LOCTEXT_NAMESPACE // "AnimDataController"
