// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimSequencerDataModel.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "ChangeTransactor.h"

#include "AnimSequencerController.generated.h"

UCLASS(BlueprintType)
class UAnimSequencerController : public UObject, public IAnimationDataController
{
	GENERATED_BODY()
public:
	UAnimSequencerController() 
	: BracketDepth(0) 
	{}

	/** Begin UObject Interface */
#if WITH_EDITORONLY_DATA
	static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	/** End UObject overrides */

	/** Begin IAnimationDataController overrides */
	virtual void SetModel(TScriptInterface<IAnimationDataModel> InModel) override;
    virtual TScriptInterface<IAnimationDataModel> GetModelInterface() const override { return ModelInterface; }
	virtual const IAnimationDataModel* const GetModel() const override { return Model.Get(); }
	virtual void OpenBracket(const FText& InTitle, bool bShouldTransact = true) override;
	virtual void CloseBracket(bool bShouldTransact = true) override;	
	virtual void SetNumberOfFrames(FFrameNumber Length, bool bShouldTransact = true) override;
	virtual void ResizeNumberOfFrames(FFrameNumber NewLength, FFrameNumber T0, FFrameNumber T1, bool bShouldTransact = true) override;
	virtual void ResizeInFrames(FFrameNumber Length, FFrameNumber T0, FFrameNumber T1, bool bShouldTransact = true) override;
	virtual void SetFrameRate(FFrameRate FrameRate, bool bShouldTransact = true) override;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS	
	virtual void SetPlayLength(float Length, bool bShouldTransact = true) override;
	virtual void ResizePlayLength(float NewLength, float T0, float T1, bool bShouldTransact = true) override;
	virtual void Resize(float Length, float T0, float T1, bool bShouldTransact = true) override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual bool AddBoneCurve(FName BoneName, bool bShouldTransact = true) override;
	virtual int32 InsertBoneTrack(FName BoneName, int32 DesiredIndex, bool bShouldTransact = true) override;
	virtual bool RemoveBoneTrack(FName BoneName, bool bShouldTransact = true) override;
	virtual void RemoveAllBoneTracks(bool bShouldTransact = true) override;
	virtual bool SetBoneTrackKeys(FName BoneName, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys, bool bShouldTransact = true) override;
	virtual bool SetBoneTrackKeys(FName BoneName, const TArray<FVector>& PositionalKeys, const TArray<FQuat>& RotationalKeys, const TArray<FVector>& ScalingKeys, bool bShouldTransact = true) override;
	virtual bool UpdateBoneTrackKeys(FName BoneName, const FInt32Range& KeyRangeToSet, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys, bool bShouldTransact = true) override;
	virtual bool UpdateBoneTrackKeys(FName BoneName, const FInt32Range& KeyRangeToSet, const TArray<FVector>& PositionalKeys, const TArray<FQuat>& RotationalKeys, const TArray<FVector>& ScalingKeys, bool bShouldTransact = true) override;
	virtual bool AddCurve(const FAnimationCurveIdentifier& CurveId, int32 CurveFlags = 0x00000004, bool bShouldTransact = true) override;
	virtual bool DuplicateCurve(const FAnimationCurveIdentifier& CopyCurveId, const FAnimationCurveIdentifier& NewCurveId, bool bShouldTransact = true) override;
	virtual bool RemoveCurve(const FAnimationCurveIdentifier& CurveId, bool bShouldTransact = true) override;
	virtual void RemoveAllCurvesOfType(ERawCurveTrackTypes SupportedCurveType, bool bShouldTransact = true) override;
	virtual bool SetCurveFlag(const FAnimationCurveIdentifier& CurveId, EAnimAssetCurveFlags Flag, bool bState = true, bool bShouldTransact = true) override;
	virtual bool SetCurveFlags(const FAnimationCurveIdentifier& CurveId, int32 Flags, bool bShouldTransact = true) override;
	virtual bool SetTransformCurveKeys(const FAnimationCurveIdentifier& CurveId, const TArray<FTransform>& TransformValues, const TArray<float>& TimeKeys, bool bShouldTransact = true) override;	
	virtual bool SetTransformCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, const FTransform& Value, bool bShouldTransact = true) override;
	virtual bool RemoveTransformCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, bool bShouldTransact = true) override;
	virtual bool RenameCurve(const FAnimationCurveIdentifier& CurveToRenameId, const FAnimationCurveIdentifier& NewCurveId, bool bShouldTransact = true) override;
	virtual bool SetCurveColor(const FAnimationCurveIdentifier& CurveId, FLinearColor Color, bool bShouldTransact = true) override;
	virtual bool SetCurveComment(const FAnimationCurveIdentifier& CurveId, const FString& Comment, bool bShouldTransact = true) override;
	virtual bool ScaleCurve(const FAnimationCurveIdentifier& CurveId, float Origin, float Factor, bool bShouldTransact = true) override;
	virtual bool SetCurveKey(const FAnimationCurveIdentifier& CurveId, const FRichCurveKey& Key, bool bShouldTransact = true) override;	
	virtual bool RemoveCurveKey(const FAnimationCurveIdentifier& CurveId, float Time, bool bShouldTransact = true) override;
	virtual bool SetCurveKeys(const FAnimationCurveIdentifier& CurveId, const TArray<FRichCurveKey>& CurveKeys, bool bShouldTransact = true) override;
	virtual bool SetCurveAttributes(const FAnimationCurveIdentifier& CurveId, const FCurveAttributes& Attributes, bool bShouldTransact = true) override;
	virtual bool RemoveBoneTracksMissingFromSkeleton(const USkeleton* Skeleton, bool bShouldTransact = true) override;
	virtual void UpdateAttributesFromSkeleton(const USkeleton* Skeleton, bool bShouldTransact = true) override;
	virtual void NotifyPopulated() override;
	virtual void ResetModel(bool bShouldTransact = true) override;
	virtual bool AddAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, bool bShouldTransact = true) override;	
    virtual bool RemoveAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, bool bShouldTransact = true) override;
    virtual int32 RemoveAllAttributesForBone(const FName& BoneName, bool bShouldTransact = true) override;
    virtual int32 RemoveAllAttributes(bool bShouldTransact = true) override;
	virtual bool SetAttributeKey(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, const void* KeyValue, const UScriptStruct* TypeStruct, bool bShouldTransact = true) override
	{
		return SetAttributeKey_Internal(AttributeIdentifier, Time, KeyValue, TypeStruct, bShouldTransact);
	}
	virtual bool SetAttributeKeys(const FAnimationAttributeIdentifier& AttributeIdentifier, TArrayView<const float> Times, TArrayView<const void*> KeyValues, const UScriptStruct* TypeStruct, bool bShouldTransact = true) override
	{
		return SetAttributeKeys_Internal(AttributeIdentifier, Times, KeyValues, TypeStruct, bShouldTransact);
	}
    virtual bool RemoveAttributeKey(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, bool bShouldTransact = true) override;	
	virtual bool DuplicateAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier, const FAnimationAttributeIdentifier& NewAttributeIdentifier, bool bShouldTransact = true) override;
	virtual void UpdateWithSkeleton(USkeleton* TargetSkeleton, bool bShouldTransact = true) override;
	virtual void PopulateWithExistingModel(TScriptInterface<IAnimationDataModel> InModel) override;
	virtual void InitializeModel() override;
protected:
	virtual void NotifyBracketOpen() override;
	virtual void NotifyBracketClosed() override;
	/** End IAnimationDataController overrides */	
private:
	/** Internal functionality for setting Attribute curve key(s) */
	bool SetAttributeKey_Internal(const FAnimationAttributeIdentifier& AttributeIdentifier, float Time, const void* KeyValue, const UScriptStruct* TypeStruct, bool bShouldTransact = true);
	bool SetAttributeKeys_Internal(const FAnimationAttributeIdentifier& AttributeIdentifier, TArrayView<const float> Times, TArrayView<const void*> KeyValues, const UScriptStruct* TypeStruct, bool bShouldTransact = true);
	
	void RemoveUnusedControlsAndCurves() const;
	
	/** Resizes the curve/attribute data stored on the model according to the provided new length and time at which to insert or remove time */
	void ResizeCurves(float NewLength, bool bInserted, float T0, float T1, bool bShouldTransact = true);
	void ResizeAttributes(float NewLength, bool bInserted, float T0, float T1, bool bShouldTransact = true);

	/** Internal functionality which handles the Sequencer/ControlRig representation of Bone Animation data */
	bool AddBoneControl(const FName& BoneName) const;
	bool RemoveBoneControl(const FName& BoneName) const;
	bool SetBoneCurveKeys(const FName& BoneName, const TArray<FVector3f>& PositionalKeys, const TArray<FQuat4f>& RotationalKeys, const TArray<FVector3f>& ScalingKeys) const;
	bool UpdateBoneCurveKeys(const FName& BoneName, const TArray<FTransform>& Keys, const TArray<float>& TimeValue) const;
	bool RemoveBoneCurveKey(const FName& BoneName, float Time);

	/** Internal functionality which handles the Sequencer/ControlRig representation of Animation Curve data */
	bool AddCurveControl(const FName& CurveName) const;
	bool RenameCurveControl(const FName& CurveName, const FName& NewCurveName) const;
	bool RemoveCurveControl(const FName& CurveName) const;
	bool SetCurveControlKeys(const FName& CurveName, const TArray<FRichCurveKey>& CurveKeys) const;
	bool SetCurveControlKey(const FName& CurveName, const FRichCurveKey& Key, bool bUpdateKey, float& OutKeyTiming) const;
	bool RemoveCurveControlKey(const FName& CurveName, float Time) const;
	bool DuplicateCurveControl(const FName& CurveName, const FName& DuplicateCurveName) const;

	void SetMovieSceneRange(FFrameNumber InFrameNumber) const;
	void EnsureModelIsInitialized() const;

	bool IgnoreSkeletonValidation() const;
	virtual bool ShouldTransact(bool bInLocalShouldTransact) const override;
private: 
	int32 BracketDepth;
	bool bShouldTransactBracket = false;

	// UObject (typed) pointer to Model
	UPROPERTY(transient)
	TWeakObjectPtr<UAnimationSequencerDataModel> Model;
	
	UPROPERTY(transient)
	TScriptInterface<IAnimationDataModel> ModelInterface;

	friend class FAnimDataControllerTestBase;
	friend UE::Anim::FOpenBracketAction;
	friend UE::Anim::FCloseBracketAction;

	DECLARE_TRANSACTION_CONTROLLER

};
