// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequencerDataModel.h"
#include "AnimSequencerController.h"

#include "Animation/AnimDataModelHasher.h"
#include "Animation/AnimSequence.h"
#include "AnimDataController.h"
#include "Async/ParallelFor.h"
#include "IAnimationEditor.h"
#include "ControlRig.h"
#include "ControlRigObjectBinding.h"
#include "Algo/Accumulate.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Rigs/FKControlRig.h"
#include "Sequencer/MovieSceneControlRigParameterSection.h"
#include "Sequencer/MovieSceneControlRigParameterTrack.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectSaveContext.h"
#include "Animation/AnimData/AnimDataNotifications.h"

#include "AnimSequencerHelpers.h"
#include "Animation/AnimationSettings.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "UObject/UObjectThreadContext.h"
#include "Animation/AnimCurveTypes.h"
#include "Runtime/MovieScene/Private/Channels/MovieSceneCurveChannelImpl.h"
#include "Containers/StackTracker.h"
#include "AnimationCompression.h"
#include "Animation/SkeletonRemapping.h"
#include "Animation/SkeletonRemappingRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimSequencerDataModel)

#define LOCTEXT_NAMESPACE "AnimSequencerDataModel"

int32 UAnimationSequencerDataModel::ValidationMode = 0;
static FAutoConsoleVariableRef CValidationMode(
	TEXT("a.AnimSequencer.ValidationMode"),
	UAnimationSequencerDataModel::ValidationMode,
	TEXT("1 = Enables validation after operations to test data integrity against legacy version. 0 = validation disabled"));

int32 UAnimationSequencerDataModel::UseDirectFKControlRigMode = 1;
static FAutoConsoleVariableRef CVarDirectControlRigMode(
	TEXT("a.AnimSequencer.DirectControlRigMode"),
	UAnimationSequencerDataModel::UseDirectFKControlRigMode,
	TEXT("1 = FKControl rig uses Direct method for setting Control transforms. 0 = FKControl rig uses Replace method (transform offsets) for setting Control transforms"));

// Opt-in fast path for generating data model FGuid (uses UMovieScene::GetSignature() rather than per-channel iteration).
static bool GAnimDataModelUseSignatureStateHash = false;
static FAutoConsoleVariableRef CVarAnimDataModelUseSignatureStateHash(
	TEXT("a.AnimDataModel.UseSignatureStateHash"),
	GAnimDataModelUseSignatureStateHash,
	TEXT("When true, AnimationSequencerDataModel uses UMovieScene::GetSignature() for state hash rather than channel iteration."));

int32 UAnimationSequencerDataModel::LazyRigHierarchyInitializationMode = 0;
static FAutoConsoleVariableRef CVarLazyRigHierarchyInitializationMode(
	TEXT("a.AnimSequencer.LazyRigHierarchyInitMode"),
	UAnimationSequencerDataModel::LazyRigHierarchyInitializationMode,
	TEXT("0 = RigHierarchy is always initialized during PostLoad.\n1 = RigHierarchy is lazily initialized _while_ running CookCommandlet otherwise during PostLoad\n2 = RigHierarchy is always lazily initialized when required for pose evaluation or model modification _while not_ running CookCommandlet\n3 =RigHierarchy is always lazily initialized"),
	ECVF_ReadOnly);

static bool ShouldInitializeHierarchyDuringCook()
{
	return UAnimationSequencerDataModel::LazyRigHierarchyInitializationMode != 1 && UAnimationSequencerDataModel::LazyRigHierarchyInitializationMode != 3;
}

static bool ShouldInitializeHierarchyDuringPostLoad()
{
	return (ShouldInitializeHierarchyDuringCook() && IsRunningCookCommandlet()) || UAnimationSequencerDataModel::LazyRigHierarchyInitializationMode == 0;
}

void UAnimationSequencerDataModel::RemoveOutOfDateControls() const
{
	if (UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
	{
		if (UFKControlRig* ControlRig = Cast<UFKControlRig>(Section->GetControlRig()))
		{
			if (URigHierarchy* Hierarchy = GetControlRigHierarchy())
			{
				if (URigHierarchyController* Controller = Hierarchy->GetController())
				{
					TArray<FRigElementKey> ElementKeysToRemove;
					Hierarchy->ForEach<FRigControlElement>([this, Section, &ElementKeysToRemove](const FRigControlElement* ControlElement) -> bool
					{
						const bool bContainsBone = Section->HasTransformParameter(ControlElement->GetFName());
						const bool bContainsCurve = Section->HasScalarParameter(ControlElement->GetFName());

						if (!bContainsBone && !bContainsCurve)
						{
							ElementKeysToRemove.Add(ControlElement->GetKey());
						}

						return true;
					});

					Hierarchy->ForEach<FRigCurveElement>([this, &ElementKeysToRemove](const FRigCurveElement* CurveElement) -> bool
					{
						const FName TargetCurveName = CurveElement->GetFName();
						if (!LegacyCurveData.FloatCurves.ContainsByPredicate([TargetCurveName](const FFloatCurve& Curve) { return Curve.GetName() == TargetCurveName; }))
						{
							ElementKeysToRemove.Add(CurveElement->GetKey());
						}
						return true;
					});

					for (const FRigElementKey& KeyToRemove : ElementKeysToRemove)
					{
						Controller->RemoveElement(KeyToRemove);
					}

					ControlRig->RefreshActiveControls();
				}
			}
		}
	}
}

USkeleton* UAnimationSequencerDataModel::GetSkeleton() const
{
	const UAnimationAsset* AnimationAsset = CastChecked<UAnimationAsset>(GetOuter());	
	checkf(AnimationAsset, TEXT("Unable to retrieve owning AnimationAsset"));

	USkeleton* Skeleton = AnimationAsset->GetSkeleton();
	if (Skeleton == nullptr)
	{
		IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("UnableToFindSkeleton", "Unable to retrieve target USkeleton for Animation Asset ({0})"), FText::FromString(*AnimationAsset->GetPathName()));
	} 

	return Skeleton;
}

void UAnimationSequencerDataModel::InitializeRigHierarchy(UFKControlRig* FKControlRig, const USkeleton* Skeleton) const
{
	if (FKControlRig && Skeleton)
	{
		FScopeLock Lock(&EvaluationLock);
	
		UFKControlRig::FRigElementInitializationOptions InitOptions;
		InitOptions.bImportCurves = false;	
		if(UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
		{
			for (const FScalarParameterNameAndCurve& AnimCurve : Section->GetScalarParameterNamesAndCurves())
			{
				InitOptions.CurveNames.Add(UFKControlRig::GetControlTargetName(AnimCurve.ParameterName, ERigElementType::Curve));
			}

			for (const FTransformParameterNameAndCurves& BoneCurve : Section->GetTransformParameterNamesAndCurves())
			{
				InitOptions.BoneNames.Add(UFKControlRig::GetControlTargetName(BoneCurve.ParameterName, ERigElementType::Bone));
			}
		}
		InitOptions.bGenerateBoneControls = InitOptions.BoneNames.Num() > 0;
		FKControlRig->SetInitializationOptions(InitOptions);

		FKControlRig->Initialize();

		FKControlRig->SetApplyMode(UseDirectFKControlRigMode == 1 ? EControlRigFKRigExecuteMode::Direct : EControlRigFKRigExecuteMode::Replace);
		FKControlRig->SetBoneInitialTransformsFromRefSkeleton(Skeleton->GetReferenceSkeleton());
		FKControlRig->Evaluate_AnyThread();
		bRigHierarchyInitialized = true;
	}
	else if (!FKControlRig)
	{
		IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("FailedToInitHierarchyFKCR", "Unable to initialize RigHierarchy due to invalid FK ControlRig for ({0})"), FText::FromString(GetPathName()));
	}
	else if (!Skeleton)
	{
		IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("FailedToInitHierarchySkeleton", "Unable to initialize RigHierarchy due to missing USkeleton for ({0})"), FText::FromString(GetPathName()));
	}	
}

void UAnimationSequencerDataModel::InitializeFKControlRig(UFKControlRig* FKControlRig, USkeleton* Skeleton, bool bForceHierarchyInitialization /*=false*/) const
{
	checkf(FKControlRig, TEXT("Invalid FKControlRig provided"));
	if (Skeleton)
	{
		LockEvaluationAndModification();
		
		FKControlRig->SetObjectBinding(MakeShared<FControlRigObjectBinding>());
		FKControlRig->GetObjectBinding()->BindToObject(Skeleton);

		if (bForceHierarchyInitialization)
		{
			InitializeRigHierarchy(FKControlRig, Skeleton);
		}
		
		UnlockEvaluationAndModification();
	}
	else
	{
		IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("InvalidFKControlRig", "Unable to initialize FKControlRig for AnimationSequencerDataModel for {0}, provided FKControlRig is invalid"), FText::FromString(*GetAnimationSequence()->GetPathName()));
	}	
}

UControlRig* UAnimationSequencerDataModel::GetControlRig() const
{
	if(const UMovieSceneControlRigParameterTrack* Track = GetControlRigTrack())
	{
		return Track->GetControlRig();
	}

	return nullptr;
}

void UAnimationSequencerDataModel::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Pre/post load any dependencies (Sequencer objects)
		TArray<UObject*> ObjectReferences;
		FReferenceFinder(ObjectReferences, this, false, true, true, true).FindReferences(this);
		for (UObject* Dependency : ObjectReferences)
		{
			Dependency->ConditionalPreload();
			Dependency->ConditionalPostLoad();
		}

		if (const UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
		{
			if (UFKControlRig* ControlRig = Cast<UFKControlRig>(Section->GetControlRig()))
			{
				// Allocate RigHierarchy UObject on game-thread
				ControlRig->PostInitInstanceIfRequired();
				InitializeFKControlRig(ControlRig, GetSkeleton(), ShouldInitializeHierarchyDuringPostLoad());
			}
		}

		CachedRawDataGUID.Invalidate();

		RemoveOutOfDateControls();

		ValidateData();
	}
}

#if WITH_EDITORONLY_DATA
void UAnimationSequencerDataModel::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(UControlRig::StaticClass()));
}
#endif

void UAnimationSequencerDataModel::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	OutDeps.Add(MovieScene);
}

void UAnimationSequencerDataModel::PostDuplicate(bool bDuplicateForPIE)
{
	UObject::PostDuplicate(bDuplicateForPIE);

	GetNotifier().Notify(EAnimDataModelNotifyType::Populated);
}

void UAnimationSequencerDataModel::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	// Forcefully skip UMovieSceneSequence::PreSave (as it generates cooked data which will never be included at the moment)
	UMovieSceneSignedObject::PreSave(ObjectSaveContext);
}

#if WITH_EDITOR
void UAnimationSequencerDataModel::WillNeverCacheCookedPlatformDataAgain()
{
	Super::WillNeverCacheCookedPlatformDataAgain();
	// Only allow clearing hierarchy data in case we are in lazy-initialize mode
	if (!ShouldInitializeHierarchyDuringCook())
    {    
	    ClearControlRigData();
    }
}

void UAnimationSequencerDataModel::PreEditUndo()
{
	Super::PreEditUndo();

	// Lock evaluation as underlying MovieScene will be modified by undo/redo. Async compression tasks
	// will be kicked off post-transaction of the model, but the underlying MovieScene may be 
	// transacted after the model and modified concurrently with a compression task.
	// We can do this because PreEditUndo calls are called on all objects in a transaction prior to 
	// its application
	LockEvaluationAndModification();
}

void UAnimationSequencerDataModel::PostEditUndo()
{
	// Unlock evaluation to allow for compression/evaluation now modifications are complete
	UnlockEvaluationAndModification();

	Super::PostEditUndo();
}
#endif

double UAnimationSequencerDataModel::GetPlayLength() const
{
	ValidateSequencerData();
	if (MovieScene)
	{
		return MovieScene->GetDisplayRate().AsSeconds(GetNumberOfFrames());
	}
	
	return 0.f;
}

int32 UAnimationSequencerDataModel::GetNumberOfFrames() const
{	
	ValidateSequencerData();
	if (MovieScene)
	{
		const TRange<FFrameNumber> FrameRange = MovieScene->GetPlaybackRange();	
		const TRangeBound<FFrameNumber>& UpperRange = FrameRange.GetUpperBound();
		const bool bInclusive = UpperRange.IsInclusive();
		int32 Value = UpperRange.GetValue().Value;
		if (!bInclusive)
		{		
			Value = FMath::Max(Value - 1, 1);
		}

		return Value;
	}
	
	return 0;
}

int32 UAnimationSequencerDataModel::GetNumberOfKeys() const
{
	return GetNumberOfFrames() + 1;
}

FFrameRate UAnimationSequencerDataModel::GetFrameRate() const
{	
	ValidateSequencerData();
	if (MovieScene)
	{
		return MovieScene->GetDisplayRate();
	}

	return UAnimationSettings::Get()->GetDefaultFrameRate();
}

const TArray<FBoneAnimationTrack>& UAnimationSequencerDataModel::GetBoneAnimationTracks() const
{
	static TArray<FBoneAnimationTrack> TempTracks;
	return TempTracks;
}

const FBoneAnimationTrack& UAnimationSequencerDataModel::GetBoneTrackByIndex(int32 TrackIndex) const
{
	static FBoneAnimationTrack TempTrack;
	return TempTrack;
}

const FBoneAnimationTrack& UAnimationSequencerDataModel::GetBoneTrackByName(FName TrackName) const
{
	static FBoneAnimationTrack TempTrack;
	return TempTrack;
}

int32 UAnimationSequencerDataModel::GetBoneTrackIndex(const FBoneAnimationTrack& Track) const
{
	return INDEX_NONE;
}

int32 UAnimationSequencerDataModel::GetBoneTrackIndexByName(FName TrackName) const
{
	return INDEX_NONE;
}

bool UAnimationSequencerDataModel::IsValidBoneTrackIndex(int32 TrackIndex) const
{
	return false;
}

int32 UAnimationSequencerDataModel::GetNumBoneTracks() const
{
	ValidateSequencerData();
		
	if(const UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
	{
		return Section->GetTransformParameterNamesAndCurves().Num();
	}

	return 0;
}

void UAnimationSequencerDataModel::GetBoneTrackNames(TArray<FName>& OutNames) const
{
	if(const UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
	{
		for (const FTransformParameterNameAndCurves& TransformParameter : Section->GetTransformParameterNamesAndCurves())
		{
			OutNames.Add(UFKControlRig::GetControlTargetName(TransformParameter.ParameterName, ERigElementType::Bone));
		}
	}
}

const FAnimationCurveData& UAnimationSequencerDataModel::GetCurveData() const
{
	return LegacyCurveData;
}

int32 UAnimationSequencerDataModel::GetNumberOfTransformCurves() const
{
	return LegacyCurveData.TransformCurves.Num();
}

int32 UAnimationSequencerDataModel::GetNumberOfFloatCurves() const
{
	return LegacyCurveData.FloatCurves.Num();
}

const TArray<FFloatCurve>& UAnimationSequencerDataModel::GetFloatCurves() const
{
	return LegacyCurveData.FloatCurves;
}

const TArray<FTransformCurve>& UAnimationSequencerDataModel::GetTransformCurves() const
{
	return LegacyCurveData.TransformCurves;
}

const FAnimCurveBase* UAnimationSequencerDataModel::FindCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	switch (CurveIdentifier.CurveType)
	{
	case ERawCurveTrackTypes::RCT_Float:
		return FindFloatCurve(CurveIdentifier);
	case ERawCurveTrackTypes::RCT_Transform:
		return FindTransformCurve(CurveIdentifier);
	default:
		checkf(false, TEXT("Invalid curve identifier type"));
	}

	return nullptr;
}

const FFloatCurve* UAnimationSequencerDataModel::FindFloatCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	ensure(CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Float);
	for (const FFloatCurve& FloatCurve : GetCurveData().FloatCurves)
	{
		if (FloatCurve.GetName() == CurveIdentifier.CurveName)
		{
			return &FloatCurve;
		}
	}

	return nullptr;
}

const FTransformCurve* UAnimationSequencerDataModel::FindTransformCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	ensure(CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Transform);
    for (const FTransformCurve& TransformCurve : GetCurveData().TransformCurves)
    {
    	if (TransformCurve.GetName() == CurveIdentifier.CurveName)
    	{
    		return &TransformCurve;
    	}
    }

    return nullptr;
}

const FRichCurve* UAnimationSequencerDataModel::FindRichCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	const FRichCurve* RichCurve = nullptr;

	if (CurveIdentifier.IsValid())
	{
		if (CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Float)
		{
			const FFloatCurve* Curve = FindFloatCurve(CurveIdentifier);
			if (Curve)
			{
				RichCurve = &Curve->FloatCurve;
			}
		}
		else if (CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Transform)
		{
			if (CurveIdentifier.Channel != ETransformCurveChannel::Invalid && CurveIdentifier.Axis != EVectorCurveChannel::Invalid)
			{
				// Dealing with transform curve
				if (const FTransformCurve* TransformCurve = FindTransformCurve(CurveIdentifier))
				{
					if (const FVectorCurve* VectorCurve = TransformCurve->GetVectorCurveByIndex(static_cast<int32>(CurveIdentifier.Channel)))
					{
						RichCurve = &VectorCurve->FloatCurves[static_cast<int32>(CurveIdentifier.Axis)];
					}
				}

			}
		}
	}

	return RichCurve;
}

bool UAnimationSequencerDataModel::IsValidBoneTrackName(const FName& TrackName) const
{
	ValidateSequencerData();

	if (const UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
	{
		const FName ControlName = UFKControlRig::GetControlName(TrackName, ERigElementType::Bone);
		return Section->GetTransformParameterNamesAndCurves().ContainsByPredicate([ControlName](const FTransformParameterNameAndCurves& Curve) { return Curve.ParameterName == ControlName; });
	}
	
	return false;
}

FTransform UAnimationSequencerDataModel::GetBoneTrackTransform(FName TrackName, const FFrameNumber& FrameNumber) const
{
	const TArray<FFrameNumber> FrameNumbers = { FrameNumber };
	TArray<FTransform> Transforms;	
	GenerateTransformKeysForControl(TrackName, FrameNumbers, Transforms);
	return Transforms.Num() ? Transforms[0] : FTransform::Identity;
}

void UAnimationSequencerDataModel::GetBoneTrackTransforms(FName TrackName, const TArray<FFrameNumber>& FrameNumbers, TArray<FTransform>& OutTransforms) const
{
	GenerateTransformKeysForControl(TrackName, FrameNumbers, OutTransforms);
}

void UAnimationSequencerDataModel::GetBoneTrackTransforms(FName TrackName, TArray<FTransform>& OutTransforms) const
{
	IterateTransformControlCurve(TrackName, [&OutTransforms](const FTransform& Transform, const FFrameNumber& FrameNumber) -> void
	{
		OutTransforms.Add(Transform);
	});
}

void UAnimationSequencerDataModel::GetBoneTracksTransform(const TArray<FName>& TrackNames, const FFrameNumber& FrameNumber, TArray<FTransform>& OutTransforms) const
{
	const TArray<FFrameNumber> FrameNumbers = { FrameNumber };
	for (int32 EntryIndex = 0; EntryIndex < TrackNames.Num(); ++EntryIndex)
	{
		GenerateTransformKeysForControl(TrackNames[EntryIndex], FrameNumbers, OutTransforms);
	}
}

FTransform UAnimationSequencerDataModel::EvaluateBoneTrackTransform(FName TrackName, const FFrameTime& FrameTime, const EAnimInterpolationType& Interpolation) const
{
	const float Alpha = Interpolation == EAnimInterpolationType::Step ? FMath::RoundToFloat(FrameTime.GetSubFrame()) : FrameTime.GetSubFrame();

	if (FMath::IsNearlyEqual(Alpha, 1.0f))
	{
		return GetBoneTrackTransform(TrackName, FrameTime.CeilToFrame());
	}
	else if (FMath::IsNearlyZero(Alpha))
	{
		return GetBoneTrackTransform(TrackName, FrameTime.FloorToFrame());
	}
	
	const FTransform From = GetBoneTrackTransform(TrackName, FrameTime.FloorToFrame());
	const FTransform To = GetBoneTrackTransform(TrackName, FrameTime.CeilToFrame());

	FTransform Blend;
	Blend.Blend(From, To, Alpha);
	return Blend;
}

const FAnimCurveBase& UAnimationSequencerDataModel::GetCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	const FAnimCurveBase* CurvePtr = FindCurve(CurveIdentifier);
	checkf(CurvePtr, TEXT("Tried to retrieve non-existing curve"));
	return *CurvePtr;
}

const FFloatCurve& UAnimationSequencerDataModel::GetFloatCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	const FFloatCurve* CurvePtr = FindFloatCurve(CurveIdentifier);
	checkf(CurvePtr, TEXT("Tried to retrieve non-existing curve"));
	return *CurvePtr;
}

const FTransformCurve& UAnimationSequencerDataModel::GetTransformCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	const FTransformCurve* CurvePtr = FindTransformCurve(CurveIdentifier);
	checkf(CurvePtr, TEXT("Tried to retrieve non-existing curve"));
	return *CurvePtr;
}

const FRichCurve& UAnimationSequencerDataModel::GetRichCurve(const FAnimationCurveIdentifier& CurveIdentifier) const
{
	const FRichCurve* CurvePtr = FindRichCurve(CurveIdentifier);
	checkf(CurvePtr, TEXT("Tried to retrieve non-existing curve"));
	return *CurvePtr;	
}

TArrayView<const FAnimatedBoneAttribute> UAnimationSequencerDataModel::GetAttributes() const
{
	return AnimatedBoneAttributes;
}

int32 UAnimationSequencerDataModel::GetNumberOfAttributes() const
{
	return AnimatedBoneAttributes.Num();
}

int32 UAnimationSequencerDataModel::GetNumberOfAttributesForBoneIndex(const int32 BoneIndex) const
{
	// Sum up total number of attributes with provided bone index
	const int32 NumberOfBoneAttributes = Algo::Accumulate<int32>(AnimatedBoneAttributes, 0, [BoneIndex](int32 Sum, const FAnimatedBoneAttribute& Attribute) -> int32
	{
		Sum += Attribute.Identifier.GetBoneIndex() == BoneIndex ? 1 : 0;
		return Sum;
	});
	return NumberOfBoneAttributes;
}

void UAnimationSequencerDataModel::GetAttributesForBone(const FName& BoneName, TArray<const FAnimatedBoneAttribute*>& OutBoneAttributes) const
{
	Algo::TransformIf(AnimatedBoneAttributes, OutBoneAttributes, [BoneName](const FAnimatedBoneAttribute& Attribute) -> bool
	{
		return Attribute.Identifier.GetBoneName() == BoneName;
	},
	[](const FAnimatedBoneAttribute& Attribute) 
	{
		return &Attribute;
	});
}

const FAnimatedBoneAttribute& UAnimationSequencerDataModel::GetAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier) const
{
	const FAnimatedBoneAttribute* AttributePtr = FindAttribute(AttributeIdentifier);
	checkf(AttributePtr, TEXT("Unable to find attribute for provided identifier"));

	return *AttributePtr;
}

const FAnimatedBoneAttribute* UAnimationSequencerDataModel::FindAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier) const
{
	return AnimatedBoneAttributes.FindByPredicate([AttributeIdentifier](const FAnimatedBoneAttribute& Attribute)
	{
		return Attribute.Identifier == AttributeIdentifier;
	});
}

UAnimSequence* UAnimationSequencerDataModel::GetAnimationSequence() const
{
	return Cast<UAnimSequence>(GetOuter());
}

template <typename HasherType>
void UAnimationSequencerDataModel::GenerateStateHash(HasherType& Hasher, const FGuidGenerationSettings& InSettings) const
{
	Hasher.UpdateString(GetClass()->GetName(), TEXT("C"));
	if (InSettings.bIncludeTimingData)
	{
		Hasher.UpdateData(GetNumberOfFrames(), TEXT("FR"));
	}

	auto UpdateWithChannel = [&Hasher](const auto& Channel, const TCHAR* Name)
		{
			Hasher.BeginObject(Name);
			Hasher.UpdateArray(Channel.GetData().GetTimes(), TEXT("T"));
			Hasher.UpdateArray(Channel.GetData().GetValues(), TEXT("V"));
			if (Channel.GetDefault().IsSet())
			{
				Hasher.UpdateData(Channel.GetDefault().GetValue(), TEXT("D"));
			}
			Hasher.EndObject();
		};

	if (const UMovieSceneControlRigParameterSection* RigSection = GetFKControlRigSection())
	{
		Hasher.BeginObject(TEXT("RIG"));

		if (!GAnimDataModelUseSignatureStateHash)
		{
			// Legacy per-channel iteration.
			UpdateWithChannel(RigSection->Weight, TEXT("W"));

			if (InSettings.bIncludeBoneData)
			{
				Hasher.BeginObject(TEXT("T"));
				for (const FTransformParameterNameAndCurves& TransformParameter : RigSection->GetTransformParameterNamesAndCurves())
				{
					Hasher.BeginObject();
					Hasher.UpdateString(TransformParameter.ParameterName.ToString(), TEXT("N"));
					for (int32 Index = 0; Index < 3; ++Index)
					{
						Hasher.BeginObject(TEXT("C"));
						UpdateWithChannel(TransformParameter.Translation[Index], TEXT("T"));
						UpdateWithChannel(TransformParameter.Rotation[Index], TEXT("R"));
						UpdateWithChannel(TransformParameter.Scale[Index], TEXT("S"));
						Hasher.EndObject();
					}
					Hasher.EndObject();
				}
				Hasher.EndObject();
			}

			if (InSettings.bIncludeCurveData)
			{
				Hasher.BeginObject(TEXT("S"));
				for (const FScalarParameterNameAndCurve& ScalarCurve : RigSection->GetScalarParameterNamesAndCurves())
				{
					Hasher.BeginObject();
					Hasher.UpdateString(ScalarCurve.ParameterName.ToString(), TEXT("N"));
					UpdateWithChannel(ScalarCurve.ParameterCurve, TEXT("C"));
					Hasher.EndObject();
				}
				Hasher.EndObject();
			}
		}
		else
		{
			checkf(MovieScene, TEXT("Missing MovieScene, indicating invalid model state"));
			Hasher.UpdateData(MovieScene->GetSignature(), TEXT("SIG"));
		}

		Hasher.EndObject();
	}

	if (InSettings.bIncludeAttributeData)
	{
		Hasher.UpdateAnimatedBoneAttributes(AnimatedBoneAttributes, TEXT("ABA"));
	}

	if (InSettings.bIncludeCurveData)
	{
		Hasher.UpdateTransformCurves(GetTransformCurves(), TEXT("TC"));
	}
}

FGuid UAnimationSequencerDataModel::GenerateGuid(const FGuidGenerationSettings& InSettings) const
{
	if (CachedRawDataGUID.IsValid())
	{
		return CachedRawDataGUID;
	}

	UE::Anim::DataModel::FHasherSha Hasher;
	GenerateStateHash(Hasher, InSettings);
	return Hasher.FinalGuid();
}

#if WITH_EDITOR
FString UAnimationSequencerDataModel::GenerateDebugStateString() const
{
	UE::Anim::DataModel::FHasherCopyToText Hasher;
	GenerateStateHash(Hasher, FGuidGenerationSettings());
	return Hasher.GetString();
}
#endif


TScriptInterface<IAnimationDataController> UAnimationSequencerDataModel::GetController()
{
	TScriptInterface<IAnimationDataController> Controller = nullptr;
#if WITH_EDITOR
	Controller = NewObject<UAnimSequencerController>();
	Controller->SetModel(this);
#endif // WITH_EDITOR

	return Controller;
}

IAnimationDataModel::FModelNotifier& UAnimationSequencerDataModel::GetNotifier()
{
	if (!Notifier)
	{
		Notifier.Reset(new IAnimationDataModel::FModelNotifier(this));
	}
	
	return *Notifier.Get();
}

void UAnimationSequencerDataModel::Evaluate(FAnimationPoseData& InOutPoseData, const UE::Anim::DataModel::FEvaluationContext& EvaluationContext) const
{
	QUICK_SCOPE_CYCLE_COUNTER(AnimationDataSequence_Evaluate);

	if(!!ValidationMode)
	{
		ValidateSequencerData();
	}

	if (UMovieSceneControlRigParameterTrack* Track = GetControlRigTrack())
	{
		FScopeLock Lock(&EvaluationLock);

		if(!bRigHierarchyInitialized)
		{
			if (UFKControlRig* FKControlRig = Cast<UFKControlRig>(GetControlRig()))
			{
				InitializeRigHierarchy(FKControlRig, GetSkeleton());
			}
		}
		
		// Evaluates and applies control curves from track to ControlRig
		EvaluateTrack(Track, EvaluationContext);

		// Generate/populate the output animation pose data
		UControlRig* ControlRig = Track->GetControlRig();
		GeneratePoseData(ControlRig, InOutPoseData, EvaluationContext);
	}
}

void UAnimationSequencerDataModel::OnNotify(const EAnimDataModelNotifyType& NotifyType, const FAnimDataModelNotifPayload& Payload)
{
	Collector.Handle(NotifyType);

	if (bPopulated)
	{
		// Once the model has been populated and a modification is made - invalidate the cached GUID
		auto ResetCachedGUID = [this]()
		{
			// Prevent reset when being populated inside of upgrade path (always happens in UAnimSequenceBase::PostLoad)
			if (!Collector.Contains(EAnimDataModelNotifyType::Populated) || !FUObjectThreadContext::Get().IsRoutingPostLoad)
			{
			    if (CachedRawDataGUID.IsValid())
			    {
				    CachedRawDataGUID.Invalidate();
			    }

			    // Mark MovieScene signature as changed to invalidate model FGuid (used for AnimSequence DDC key hash)
			    if (GAnimDataModelUseSignatureStateHash)
			    {
					checkf(MovieScene, TEXT("Missing MovieScene, indicating invalid model state"));
				    MovieScene->MarkAsChanged();
			    }
			}
		};

		bool bRefreshed = false;
		auto RefreshControlsAndProxy = [this, &bRefreshed]()
        {
			if (!bRefreshed)
			{
				if (UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
				{
					if (!IsRunningCookCommandlet())
					{
						Section->ReconstructChannelProxy();
					}
			    
					if (UFKControlRig* FKRig = Cast<UFKControlRig>(Section->GetControlRig()))
					{
						if (!bRigHierarchyInitialized)
						{
							InitializeRigHierarchy(FKRig, GetSkeleton());
						}

						FKRig->RefreshActiveControls();
					}
				}
				bRefreshed = true;
			}
        };	

		if (Collector.IsNotWithinBracket())
		{
			const TArray<EAnimDataModelNotifyType> CurveStorageNotifyTypes = {EAnimDataModelNotifyType::CurveAdded, EAnimDataModelNotifyType::CurveChanged, EAnimDataModelNotifyType::CurveRenamed, EAnimDataModelNotifyType::CurveRemoved,
			EAnimDataModelNotifyType::CurveScaled, EAnimDataModelNotifyType::Populated, EAnimDataModelNotifyType::Reset };

			if(Collector.Contains(CurveStorageNotifyTypes))
			{
				if(!ValidationMode)
				{
					RegenerateLegacyCurveData();
				}
				RefreshControlsAndProxy();
				ResetCachedGUID();
			}

			const TArray<EAnimDataModelNotifyType> CurveDataNotifyTypes = {EAnimDataModelNotifyType::CurveFlagsChanged, EAnimDataModelNotifyType::CurveColorChanged, EAnimDataModelNotifyType::CurveCommentChanged};
			if(Collector.Contains(CurveDataNotifyTypes))
			{
				if(!ValidationMode)
				{
					UpdateLegacyCurveData();
				}
				RefreshControlsAndProxy();
				ResetCachedGUID();
			}

			const TArray<EAnimDataModelNotifyType> BonesNotifyTypes = {EAnimDataModelNotifyType::TrackAdded, EAnimDataModelNotifyType::TrackChanged, EAnimDataModelNotifyType::TrackRemoved, EAnimDataModelNotifyType::Populated, EAnimDataModelNotifyType::Reset };
			if(Collector.Contains(BonesNotifyTypes))
			{
				RefreshControlsAndProxy();
				ResetCachedGUID();
			}

			if (Collector.Contains(EAnimDataModelNotifyType::Populated))
			{
				RefreshControlsAndProxy();
			}
		}
		else
		{
			// These changes can cause subsequent evaluation to fail due to mismatching data (related to changed controls)
			const TArray<EAnimDataModelNotifyType> RigModificationTypes = {EAnimDataModelNotifyType::TrackAdded, EAnimDataModelNotifyType::TrackRemoved, EAnimDataModelNotifyType::CurveAdded, EAnimDataModelNotifyType::CurveRenamed, EAnimDataModelNotifyType::CurveRemoved};
			if(Collector.Contains(RigModificationTypes))
			{
				RefreshControlsAndProxy();
			}
		}
				
		ValidateData();
	}
}

UMovieSceneControlRigParameterTrack* UAnimationSequencerDataModel::GetControlRigTrack() const
{
	if (!MovieScene)
	{
		return nullptr;
	}

	// Iterate tracks internally rather than using FindTrack to by-pass IsValid check, this function can be invoked during GC mark pass
	for (UMovieSceneTrack* Track : MovieScene->GetTracks())
	{
		if (UMovieSceneControlRigParameterTrack* ControlRigTrack = Cast<UMovieSceneControlRigParameterTrack>(Track))
		{
			return ControlRigTrack;
		}
	}
	return nullptr;
}

UMovieSceneControlRigParameterSection* UAnimationSequencerDataModel::GetFKControlRigSection() const
{
	if (MovieScene)
	{
		if(const UMovieSceneControlRigParameterTrack* Track = GetControlRigTrack())
		{
			for (UMovieSceneSection* TrackSection : Track->GetAllSections())
			{
				if (UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(TrackSection))
				{
					if (const UControlRig* ControlRig = Section->GetControlRig())
					{
						if (ControlRig->IsA<UFKControlRig>())
						{
							return Section;
						}
					}
				}
			}
		}
	}

	return nullptr;
}

URigHierarchy* UAnimationSequencerDataModel::GetControlRigHierarchy() const
{	
	if (const UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
	{
		UControlRig* ControlRig = Section->GetControlRig();
		if (ControlRig)
		{
			if (UFKControlRig* FKRig = Cast<UFKControlRig>(ControlRig))
			{
				if (!bRigHierarchyInitialized)
				{
					InitializeRigHierarchy(FKRig, GetSkeleton());
				}

				return FKRig->GetHierarchy();
			}

			IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("UnableToFindRigHierarchy", "Unable to retrieve RigHierarchy for ControlRig ({0})"), FText::FromString(ControlRig->GetPathName()));
		}
	}

	return nullptr;
}

void UAnimationSequencerDataModel::RegenerateLegacyCurveData()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RegenerateLegacyCurveData);
	ValidateSequencerData();

	if (const UMovieSceneControlRigParameterTrack* Track = GetControlRigTrack())
	{
		for (const UMovieSceneSection* TrackSection : Track->GetAllSections())
		{
			if (const UMovieSceneControlRigParameterSection* Section = Cast<UMovieSceneControlRigParameterSection>(TrackSection))
			{
				if (URigHierarchy* Hierarchy = GetControlRigHierarchy())
				{
					const TArray<FScalarParameterNameAndCurve>& ScalarCurves = Section->GetScalarParameterNamesAndCurves();
					LegacyCurveData.FloatCurves.Empty();

					Hierarchy->ForEach<FRigCurveElement>([Hierarchy, this, ScalarCurves, FrameRate = GetFrameRate()](const FRigCurveElement* CurveElement) -> bool
					{
						const FRigElementKey ControlKey(UFKControlRig::GetControlName(CurveElement->GetFName(), ERigElementType::Curve), ERigElementType::Control);
						if (const FRigControlElement* Element = Hierarchy->Find<FRigControlElement>(ControlKey))
						{
							FFloatCurve& FloatCurve = LegacyCurveData.FloatCurves.AddDefaulted_GetRef();
							FloatCurve.SetName(CurveElement->GetFName());
							FloatCurve.Color = Element->Settings.ShapeColor;

							const FAnimationCurveIdentifier CurveId(FloatCurve.GetName(), ERawCurveTrackTypes::RCT_Float);
							if (CurveIdentifierToMetaData.Contains(CurveId))
							{
								const FAnimationCurveMetaData& CurveMetaData = CurveIdentifierToMetaData.FindChecked(CurveId);
								FloatCurve.SetCurveTypeFlags(CurveMetaData.Flags);
								FloatCurve.Color = CurveMetaData.Color;
								FloatCurve.Comment = CurveMetaData.Comment;
							}

							if (const FScalarParameterNameAndCurve* ScalarCurve = ScalarCurves.FindByPredicate([Element](const FScalarParameterNameAndCurve& Curve)
								{
									return Curve.ParameterName == Element->GetFName();
								}))
							{
								AnimSequencerHelpers::ConvertFloatChannelToRichCurve(ScalarCurve->ParameterCurve, FloatCurve.FloatCurve, FrameRate);
							}
						}
						return true;
					});
				}
			}
		}
	}
}

void UAnimationSequencerDataModel::UpdateLegacyCurveData()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateLegacyCurveData);
	ValidateSequencerData();

	if (const URigHierarchy* Hierarchy = GetControlRigHierarchy())
	{
		for (FFloatCurve& FloatCurve : LegacyCurveData.FloatCurves)
		{
			const FRigElementKey ControlKey(UFKControlRig::GetControlName(FloatCurve.GetName(), ERigElementType::Curve), ERigElementType::Control);
			if (const FRigControlElement* Element = Hierarchy->Find<FRigControlElement>(ControlKey))
			{
				FloatCurve.Color = Element->Settings.ShapeColor;

				const FAnimationCurveIdentifier CurveId(FloatCurve.GetName(), ERawCurveTrackTypes::RCT_Float);
				if (const FAnimationCurveMetaData* CurveMetaData = CurveIdentifierToMetaData.Find(CurveId))
				{
					FloatCurve.SetCurveTypeFlags(CurveMetaData->Flags);
					FloatCurve.Color = CurveMetaData->Color;
					FloatCurve.Comment = CurveMetaData->Comment;
				}
			}
		}
	}
}

void UAnimationSequencerDataModel::ValidateData() const
{		
	ValidateSequencerData();
	ValidateControlRigData();

	if (!!ValidationMode)
	{
		ValidateLegacyAgainstControlRigData();
	}
}

void UAnimationSequencerDataModel::ValidateSequencerData() const
{
	if (MovieScene)
	{
		const int32 NumberOfTracks = MovieScene->GetTracks().Num();
		if (NumberOfTracks == 1)
		{
			if (const UMovieSceneControlRigParameterTrack* Track = GetControlRigTrack())
			{
				const int32 NumberOfSections = Track->GetAllSections().Num();
				if (NumberOfSections == 1)
				{
					if(const UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
					{
						if (const UControlRig* ControlRig = Section->GetControlRig())
						{
							if (!ControlRig->IsA<UFKControlRig>())
							{
								IAnimationDataController::ReportMessage(this, FText::Format(LOCTEXT("InvalidControlRigClass", "Unexpected class {0} on ControlRig, expecting FKControlRig"), FText::FromString(ControlRig->GetAssetReference().GetPathName())), ELogVerbosity::Error);
							}
						}
						else
						{
							IAnimationDataController::ReportMessage(this, LOCTEXT("MissingControlRig", "Unable to find Control Rig"), ELogVerbosity::Error);
						}
					}
					else
					{
						IAnimationDataController::ReportMessage(this, LOCTEXT("MissingControlRigSection", "Unable to find Control Rig Section"), ELogVerbosity::Error);	
					}
				}
				else
				{
					IAnimationDataController::ReportMessage(this, FText::Format(LOCTEXT("InvalidNumberOfSections", "Invalid number of Sections found for Control Rig Track expected 1 but found {0}"), FText::AsNumber(NumberOfSections)), ELogVerbosity::Error);
				}
			}
			else
			{
				IAnimationDataController::ReportMessage(this, LOCTEXT("MissingControlRigTrack", "Unable to find Control Rig Track"), ELogVerbosity::Error);	
			}
		}
		else
		{
			IAnimationDataController::ReportMessage(this, FText::Format(LOCTEXT("InvalidNumberOfTracks", "Invalid number of Tracks in Movie Scene expected 1 but found {0}"), FText::AsNumber(NumberOfTracks)), ELogVerbosity::Error);
		}
	}
	else
	{
		IAnimationDataController::ReportMessage(this, LOCTEXT("MissingMovieScene", "No Movie Scene found for SequencerDataModel"), ELogVerbosity::Error);
	}
}

void UAnimationSequencerDataModel::ValidateControlRigData() const
{
	if(const UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
	{
		if (UControlRig* ControlRig = Section->GetControlRig())
		{
			if (ControlRig->IsA<UFKControlRig>())
			{
				if (bRigHierarchyInitialized)
				{
					const URigHierarchy* Hierarchy = ControlRig->GetHierarchy();
					if (Hierarchy && ValidationMode)
					{
						// Validate Rig Hierarchy against the outer Animation Sequence its (reference) Skeleton		
						if (const USkeleton* Skeleton = GetSkeleton())
						{
							const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
							const int32 NumberOfBones = ReferenceSkeleton.GetNum();

							// Validating the bone elements against the reference skeleton bones
							for (int32 BoneIndex = 0; BoneIndex < NumberOfBones; ++BoneIndex)
							{
								const FName ExpectedBoneName = ReferenceSkeleton.GetBoneName(BoneIndex);
								const bool bIsVirtualBone = ExpectedBoneName.ToString().StartsWith(VirtualBoneNameHelpers::VirtualBonePrefix);
								if (!bIsVirtualBone)
								{
									const FRigElementKey BoneKey(ExpectedBoneName, ERigElementType::Bone);
									const FRigBoneElement* BoneElement = Hierarchy->Find<FRigBoneElement>(BoneKey);
									checkf(BoneElement, TEXT("Unable to find FRigBoneElement in RigHierarchy for Bone with name: %s"), *ExpectedBoneName.ToString());

									const int32 ParentBoneIndex = ReferenceSkeleton.GetParentIndex(BoneIndex);
									if (BoneElement && ParentBoneIndex != INDEX_NONE)
									{
										const FName ExpectedParentBoneName = ReferenceSkeleton.GetBoneName(ParentBoneIndex);
										const FRigElementKey ParentBoneKey(ExpectedParentBoneName, ERigElementType::Bone);

										const FRigBoneElement* ParentBoneElement = Hierarchy->Find<FRigBoneElement>(ParentBoneKey);
										checkf(BoneElement->ParentElement == ParentBoneElement, TEXT("Unexpected Parent Element for Bone %s. Expected %s but found %s"), *ExpectedBoneName.ToString(), *ExpectedParentBoneName.ToString(), *ParentBoneElement->GetDisplayName().ToString());
									}	
								}
							}
						}
					}
					else if (!Hierarchy)
					{
						IAnimationDataController::ReportMessage(this, LOCTEXT("MissingHierarchy", "Unable to retrieve Control Rig Hierarchy"), ELogVerbosity::Error);	
					}
				}
			}
			else
			{
				IAnimationDataController::ReportMessage(this, FText::Format(LOCTEXT("InvalidControlRigClass", "Unexpected class {0} on ControlRig, expecting FKControlRig"), FText::FromString(ControlRig->GetAssetReference().GetPathName())), ELogVerbosity::Error);	
			}
		}
		else
		{
			IAnimationDataController::ReportMessage(this, LOCTEXT("MissingControlRig", "Unable to find Control Rig"), ELogVerbosity::Error);	
		}
	}
	else
	{
		IAnimationDataController::ReportMessage(this, LOCTEXT("MissingControlRigSection", "Unable to find Control Rig Section"), ELogVerbosity::Error);	
	}
}

void UAnimationSequencerDataModel::ValidateLegacyAgainstControlRigData() const
{
	if (bRigHierarchyInitialized)
	{
		UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection();

		UControlRig* ControlRig = Section->GetControlRig();
		URigHierarchy* Hierarchy = ControlRig->GetHierarchy();

		// Validate bone tracks against controls
		const UAnimSequence* OuterSequence = GetAnimationSequence();
		if (const USkeleton* Skeleton = OuterSequence->GetSkeleton())
		{
			const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();	
			// Validate curve data against controls
			for (const FFloatCurve& FloatCurve : LegacyCurveData.FloatCurves)
			{
				const FName CurveName = FloatCurve.GetName();					
				const FRigElementKey CurveKey(CurveName, ERigElementType::Curve);
				const FRigCurveElement* CurveElement = Hierarchy->Find<FRigCurveElement>(CurveKey);
				if (!CurveElement)
				{
					IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("CurveElementNotFound", "Unable to find FRigCurve in RigHierarchy for Curve with name: {0}"), FText::FromName(CurveName));
				}
	        
				const FRigElementKey CurveControlKey(UFKControlRig::GetControlName(CurveName, ERigElementType::Curve), ERigElementType::Control);
				const FRigControlElement* CurveControlElement = Hierarchy->Find<FRigControlElement>(CurveControlKey);
				if (!CurveControlElement)
				{
					IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("CurveControlElementNotFound", "Unable to find FRigControlElement in RigHierarchy for Curve with name: {0}"), FText::FromName(CurveName));
				}
			
				const FScalarParameterNameAndCurve* CurveControlParameter = Section->GetScalarParameterNamesAndCurves().FindByPredicate([CurveControlKey](const FScalarParameterNameAndCurve& ParameterPair)
				{
					return ParameterPair.ParameterName == CurveControlKey.Name;
				});
			
				if (CurveControlParameter)
				{
					for (const FRichCurveKey& Key : FloatCurve.FloatCurve.GetConstRefOfKeys())
					{
						float ParameterValue = 0.f;
						const FFrameTime FrameTime = CurveControlParameter->ParameterCurve.GetTickResolution().AsFrameTime(Key.Time);

						if (!CurveControlParameter->ParameterCurve.Evaluate(FrameTime, ParameterValue))
						{
							IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("FailedToEvaluateCurveControl", "Unable to evaluate Control Curve ({0}) at interval {1}"), FText::FromName(CurveName), FText::AsNumber(FrameTime.AsDecimal()));
						}

						const float RichCurveValue = FloatCurve.FloatCurve.Eval(Key.Time);
						// QQ threshold
						if (!(FMath::IsNearlyEqual(ParameterValue, Key.Value, 0.001f) || FMath::IsNearlyEqual(ParameterValue, RichCurveValue, 0.001f)))
						{
							IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("CurveDeviationError", "Unexpected Control Curve ({0}) evaluation value {1} at {2}, expected {3} ({4})"), FText::FromName(CurveName), FText::AsNumber(ParameterValue), FText::AsNumber(FrameTime.AsDecimal()), FText::AsNumber(Key.Value), FText::AsNumber(RichCurveValue));
						}
					}
				}
				else
				{
					IAnimationDataController::ReportObjectErrorf(this, LOCTEXT("ParameterNotFound", "Unable to find FScalarParameterNameAndCurve in RigHierarchy for Curve Control with name: {0}"), FText::FromName(CurveName));
				}
			}	
		}
	}
}

void UAnimationSequencerDataModel::IterateTransformControlCurve(const FName& BoneName, TFunction<void(const FTransform&, const FFrameNumber&)> IterationFunction, const TArray<FFrameNumber>* InFrameNumbers /*= nullptr*/) const
{
	ValidateSequencerData();
	ValidateControlRigData();

	if (const UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
	{
		const FRigElementKey BoneControlKey(UFKControlRig::GetControlName(BoneName, ERigElementType::Bone), ERigElementType::Control);
		if (const FTransformParameterNameAndCurves* ControlCurvePtr = Section->GetTransformParameterNamesAndCurves().FindByPredicate([CurveName = BoneControlKey.Name](const FTransformParameterNameAndCurves& TransformParameter)
		{
			return TransformParameter.ParameterName == CurveName;
		}))
		{
			const FTransformParameterNameAndCurves& ControlCurve = *ControlCurvePtr;

			FVector3f Location(0.f);
			FVector3f EulerAngles(0.f);
			FVector3f Scale = FVector3f::OneVector;

			// Check whether or not any data is contained
			bool bContainsData = false;
			bool bContainsKeys = false;
			for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
			{
				bContainsData |= ControlCurve.Translation[ChannelIndex].HasAnyData();
				bContainsKeys |= ControlCurve.Translation[ChannelIndex].GetNumKeys() != 0;
				bContainsData |= ControlCurve.Rotation[ChannelIndex].HasAnyData();		
				bContainsKeys |= ControlCurve.Rotation[ChannelIndex].GetNumKeys() != 0;		
				bContainsData |= ControlCurve.Scale[ChannelIndex].HasAnyData();
				bContainsKeys |= ControlCurve.Scale[ChannelIndex].GetNumKeys() != 0;
			}

			if (bContainsData)
			{
				const int32 NumberOfKeysToIterate = InFrameNumbers != nullptr ? InFrameNumbers->Num() : GetNumberOfKeys();
				for (int32 KeyIndex = 0; KeyIndex < NumberOfKeysToIterate; ++KeyIndex)
				{
					const FFrameNumber Frame = InFrameNumbers != nullptr ? (*InFrameNumbers)[KeyIndex] : FFrameNumber(KeyIndex);
					for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
					{
						ControlCurve.Translation[ChannelIndex].Evaluate(Frame, Location[ChannelIndex]);
						ControlCurve.Rotation[ChannelIndex].Evaluate(Frame, EulerAngles[ChannelIndex]);
						ControlCurve.Scale[ChannelIndex].Evaluate(Frame, Scale[ChannelIndex]);
					}

					FTransform Transform;
					Transform.SetLocation(FVector(Location));
					Transform.SetRotation(FQuat::MakeFromEuler(FVector(EulerAngles)));
					Transform.SetScale3D(FVector(Scale));

					Transform.NormalizeRotation();

					IterationFunction(Transform, Frame);
				}
			}
		}
	}
}

void UAnimationSequencerDataModel::GenerateTransformKeysForControl(const FName& BoneName, TArray<FTransform>& InOutTransforms, TArray<FFrameNumber>& InOutFrameNumbers) const
{
	IterateTransformControlCurve(BoneName, [&InOutTransforms, &InOutFrameNumbers](const FTransform& Transform, const FFrameNumber& FrameNumber) -> void
	{
		InOutTransforms.Add(Transform);
		InOutFrameNumbers.Add(FrameNumber);
	});
}

void UAnimationSequencerDataModel::GenerateTransformKeysForControl(const FName& BoneName, const TArray<FFrameNumber>& FrameNumbers, TArray<FTransform>& InOutTransforms) const
{
	IterateTransformControlCurve(BoneName, [&InOutTransforms](const FTransform& Transform, const FFrameNumber& FrameNumber) -> void
	{
		InOutTransforms.Add(Transform);
	}, &FrameNumbers);
}

void UAnimationSequencerDataModel::ClearControlRigData()
{
	if (bRigHierarchyInitialized)
	{
		FScopeLock Lock(&EvaluationLock);
		if (UFKControlRig* FKControlRig = Cast<UFKControlRig>(GetControlRig()))
		{
			FKControlRig->GetHierarchy()->Reset();
			bRigHierarchyInitialized = false;
		}
	}	
}

UMovieScene* UAnimationSequencerDataModel::GetMovieScene() const
{
	return MovieScene;
}

UObject* UAnimationSequencerDataModel::GetParentObject(UObject* MovieSceneBlends) const
{
	return GetOuter();
}

void UAnimationSequencerDataModel::GeneratePoseData(UControlRig* ControlRig, FAnimationPoseData& InOutPoseData, const UE::Anim::DataModel::FEvaluationContext& EvaluationContext) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GeneratePoseData);
	
	if (ControlRig)
	{
		if (const URigHierarchy* RigHierarchy = ControlRig->GetHierarchy())
		{
			// Evaluate Control rig to update bone and curve elements according to controls
			ControlRig->Evaluate_AnyThread();

			// Start with ref-pose
			FCompactPose& RigPose = InOutPoseData.GetPose();
			RigPose.ResetToRefPose();
			const FBoneContainer& RequiredBones = RigPose.GetBoneContainer();
			FBlendedCurve& Curve = InOutPoseData.GetCurve();
			Curve.Empty();

			UE::Anim::Retargeting::FRetargetingScope RetargetingScope(GetSkeleton(), RigPose, EvaluationContext);
			
			const FReferenceSkeleton& MeshRefSkeleton = RequiredBones.GetReferenceSkeleton();
			// Called during compression that can occur while GC is in progress, marking weakptrs as unreachable temporarily
			const FReferenceSkeleton& SkeletonRefSkeleton = RequiredBones.GetSkeletonAsset(true)->GetReferenceSkeleton();

			// Populate bone/curve elements to Pose/Curve indices
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_GetMappings);

				const bool bDifferentBoneContainerReferenceSkeleton = &InOutPoseData.GetPose().GetBoneContainer().GetReferenceSkeleton() != &GetSkeleton()->GetReferenceSkeleton();
				RigHierarchy->ForEach<FRigBoneElement>([this, &MeshRefSkeleton, &RequiredBones, &SkeletonRefSkeleton, &RigHierarchy, &RetargetingScope, &RigPose, bDifferentBoneContainerReferenceSkeleton](const FRigBoneElement* BoneElement) -> bool
				{
					const FName& BoneName = BoneElement->GetFName();
					const int32 BoneIndex = MeshRefSkeleton.FindBoneIndex(BoneName);
					if (BoneIndex != INDEX_NONE)
					{
						const int32 SkeletonBoneIndex = SkeletonRefSkeleton.FindBoneIndex(BoneName);
						if (SkeletonBoneIndex != INDEX_NONE)
						{
							const FCompactPoseBoneIndex CompactPoseBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonIndex(SkeletonBoneIndex);
							if (CompactPoseBoneIndex != INDEX_NONE)
							{
								const FTransform CurrentTransform = RigHierarchy->GetLocalTransform(BoneElement->GetKey());
								if (bDifferentBoneContainerReferenceSkeleton)
								{
									if (CurrentTransform.Equals(RigHierarchy->GetLocalTransform(BoneElement->GetKey(), true)))
									{
										// In case of mismatching ref-skeletons (and therefore ref-pose) only write out transforms for bones which do not match their ref transform
										// comes at a perf cost due to transform comparison
										return true;
									}
								}
								
								RetargetingScope.AddTrackedBone(CompactPoseBoneIndex, SkeletonBoneIndex);
								// Retrieve evaluated bone transform from Hierarchy
								RigPose[CompactPoseBoneIndex] = CurrentTransform;

								if (RigPose[CompactPoseBoneIndex].ContainsNaN())
								{
									IAnimationDataController::ReportObjectWarningf(this, LOCTEXT("BoneTransformNaN", "Bone transform for {0} contains NaN value, resetting to reference bone pose"), FText::FromName(BoneName));
									RigPose[CompactPoseBoneIndex] = RigHierarchy->GetLocalTransform(BoneElement->GetKey(), true);
								}
							}
						}
					}

					return true;
				});

				RigHierarchy->ForEach<FRigCurveElement>([this, &RigHierarchy, &Curve](const FRigCurveElement* CurveElement) -> bool
				{
					const FName& CurveName = CurveElement->GetFName();
				Curve.Add(CurveName, RigHierarchy->GetCurveValue(CurveElement->GetKey()));
					return true;
				});
			}

			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_NormalizeRotations);
				RigPose.NormalizeRotations();
			}

			// Apply any additive transform curves - if requested and any are set
			if (!RequiredBones.ShouldUseSourceData())
			{
				for (const FTransformCurve& TransformCurve : GetTransformCurves())
				{
					// if disabled, do not handle
					if (TransformCurve.GetCurveTypeFlag(AACF_Disabled))
					{
						continue;
					}
			
					// Add or retrieve curve
					const FName& CurveName = TransformCurve.GetName();
					// note we're not checking Curve.GetCurveTypeFlags() yet
					FTransform Value = TransformCurve.Evaluate(static_cast<float>(EvaluationContext.SampleFrameRate.AsSeconds(EvaluationContext.SampleTime)), 1.f);

					const FSkeletonPoseBoneIndex SkeletonBoneIndex = FSkeletonPoseBoneIndex(SkeletonRefSkeleton.FindBoneIndex(CurveName));
					if (SkeletonBoneIndex != INDEX_NONE)
					{
						const FCompactPoseBoneIndex BoneIndex(RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(SkeletonBoneIndex));
						if(BoneIndex != INDEX_NONE)
						{
							const FTransform LocalTransform = RigPose[BoneIndex];
							RigPose[BoneIndex].SetRotation(LocalTransform.GetRotation() * Value.GetRotation());
							RigPose[BoneIndex].SetTranslation(LocalTransform.TransformPosition(Value.GetTranslation()));
							RigPose[BoneIndex].SetScale3D(LocalTransform.GetScale3D() * Value.GetScale3D());
							if (RigPose[BoneIndex].ContainsNaN())
							{
								IAnimationDataController::ReportObjectWarningf(this, LOCTEXT("TransformCurveBoneNaN", "Applying transform curve {0} results in NaN value, reverting back to animated bone pose"), FText::FromName(CurveName));
								RigPose[BoneIndex] = LocalTransform;
							}
						}
					}
					else
					{
						IAnimationDataController::ReportObjectWarningf(this, LOCTEXT("TransformCurveBoneNotFound", "Failed to find BoneIndex for transform curve {0}"), FText::FromName(CurveName));
					}
				}
			}

			// Generate relative transform for VirtualBones according to source/target
			{					
				QUICK_SCOPE_CYCLE_COUNTER(STAT_GenerateVirtualBones);
				
				TArray<FVirtualBoneCompactPoseData>& VBCompactPoseData = UE::Anim::FBuildRawPoseScratchArea::Get().VirtualBoneCompactPoseData;
				VBCompactPoseData = RequiredBones.GetVirtualBoneCompactPoseData();
				if (VBCompactPoseData.Num() > 0)
				{
					FCSPose<FCompactPose> CSPose1;
					CSPose1.InitPose(RigPose);

					for (const FVirtualBoneCompactPoseData& VB : VBCompactPoseData)
					{
						const FTransform Source = CSPose1.GetComponentSpaceTransform(VB.SourceIndex);
						const FTransform Target = CSPose1.GetComponentSpaceTransform(VB.TargetIndex);
						const FTransform RelativeTransform = Target.GetRelativeTransform(Source);

						if (RelativeTransform.ContainsNaN())
						{
							const FSkeletonPoseBoneIndex SkeletonIndex = RequiredBones.GetSkeletonPoseIndexFromCompactPoseIndex(VB.VBIndex);
							ensure(SkeletonIndex != INDEX_NONE);
							const FName VBName = RequiredBones.GetReferenceSkeleton().GetBoneName(SkeletonIndex.GetInt());							
							IAnimationDataController::ReportObjectWarningf(this, LOCTEXT("VirtualBoneTransformNaN", "Virtual Bone transform for {0} contains NaN value, ignoring calculated pose"), FText::FromName(VBName));
							continue;
						}

						RigPose[VB.VBIndex] = RelativeTransform;
					}
				}
			}

			{				
				QUICK_SCOPE_CYCLE_COUNTER(STAT_SetAttributes);
				const FSkeletonRemapping& SkeletonRemapping = UE::Anim::FSkeletonRemappingRegistry::Get().GetRemapping(GetSkeleton(), RequiredBones.GetSkeletonAsset());

				// Evaluate attributes at requested time interval
				for (const FAnimatedBoneAttribute& Attribute : AnimatedBoneAttributes)
				{
					const int32 AttributeBoneIndex = Attribute.Identifier.GetBoneIndex();
					const int32 RemappedSkeletonBoneIndex = SkeletonRemapping.IsValid() ? SkeletonRemapping.GetTargetSkeletonBoneIndex(AttributeBoneIndex) : AttributeBoneIndex;
					const FCompactPoseBoneIndex PoseBoneIndex = RequiredBones.GetCompactPoseIndexFromSkeletonPoseIndex(FSkeletonPoseBoneIndex(RemappedSkeletonBoneIndex));
					// Only add attribute if the bone its tied to exists in the currently evaluated set of bones
					if(PoseBoneIndex.IsValid())
					{
						UE::Anim::Attributes::GetAttributeValue(InOutPoseData.GetAttributes(), PoseBoneIndex, Attribute, EvaluationContext.SampleFrameRate.AsSeconds(EvaluationContext.SampleTime));
					}
				}				
			}			
		}
	}
}

void UAnimationSequencerDataModel::EvaluateTrack(UMovieSceneControlRigParameterTrack* CR_Track, const UE::Anim::DataModel::FEvaluationContext& EvaluationContext) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_EvaluateTrack);
	
	// Determine frame-time to sample according to the interpolation type (floor to frame for step interpolation)
	const FFrameTime InterpolationTime = EvaluationContext.InterpolationType == EAnimInterpolationType::Step ? EvaluationContext.SampleTime.FloorToFrame() : EvaluationContext.SampleTime;
	const FFrameTime BoneSampleTime = FFrameRate::TransformTime(InterpolationTime, EvaluationContext.SampleFrameRate, MovieScene->GetTickResolution());	

	auto FindFirstActiveSectionAtTime = [CR_Track, Time = BoneSampleTime.FrameNumber]() -> UMovieSceneSection*
	{
		if (CR_Track && !CR_Track->IsEvalDisabled())
		{
			for (UMovieSceneSection* Section : CR_Track->GetAllSections())
			{
				if (Section && Section->IsActive() && !CR_Track->IsRowEvalDisabled(Section->GetRowIndex()))
				{
					if (Section->GetRange().Contains(Time))
					{
						return Section;
					}
				}
			}
		}
		return nullptr;
	};
	
	// Retrieve section within range of requested evaluation frame
	if (UMovieSceneSection* Section = FindFirstActiveSectionAtTime())
	{
		const UMovieSceneControlRigParameterSection* FKRigSection = CastChecked<UMovieSceneControlRigParameterSection>(Section);
		if (!FKRigSection->ControlRigAssetReference.IsSourceOf(UFKControlRig::StaticClass()->GetDefaultObject()))
		{
			IAnimationDataController::ReportMessage(this, FText::Format(LOCTEXT("InvalidControlRigClass", "Unexpected class {0} on ControlRig, expecting FKControlRig"), FText::FromString(FKRigSection->ControlRigAssetReference.GetPathName())), ELogVerbosity::Error);
			return;
		}
		
		bool bWasDoNotKey = false;
		bWasDoNotKey = FKRigSection->GetDoNotKey();
		FKRigSection->SetDoNotKey(true);

		UControlRig* ControlRig = FKRigSection->GetControlRig();
		check(ControlRig);

		// Reset to ref-pose
		if (URigHierarchy* RigHierarchy = ControlRig->GetHierarchy())
		{
			RigHierarchy->ResetPoseToInitial(ERigElementType::Bone);

			const TArray<FScalarParameterNameAndCurve>& ScalarParameters = FKRigSection->GetScalarParameterNamesAndCurves();
			for (const FScalarParameterNameAndCurve& TypedParameter : ScalarParameters)
			{
				const FName& Name = TypedParameter.ParameterName;
				float Value = 0.f;

				const FFrameTime CurveSampleTime = FFrameRate::TransformTime(EvaluationContext.SampleTime, EvaluationContext.SampleFrameRate, TypedParameter.ParameterCurve.GetTickResolution());
				if(TypedParameter.ParameterCurve.Evaluate(CurveSampleTime, Value))
				{					
					FRigControlElement* ControlElement = ControlRig->FindControl(Name);
					if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::Float)
					{
						RigHierarchy->SetControlValue(ControlElement, FRigControlValue::Make<float>(Value), ERigControlValueType::Current, false, true, false, false);
					}
				}
			}

			const TArray<FTransformParameterNameAndCurves>& TransformParameters = FKRigSection->GetTransformParameterNamesAndCurves();
			if (TransformParameters.Num())
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_EvaluateTransformParameters);
			
				struct FEvaluationInfo
				{
					float Interp = 0.f;
					int32 Index1 = INDEX_NONE, Index2 = INDEX_NONE;
				};

				TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::FTimeEvaluationCache FromFrameTimeEvaluationCache;
				TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::FTimeEvaluationCache ToFrameTimeEvaluationCache;

				const int32 NumberOfKeys = GetNumberOfKeys();
			
				for (const FTransformParameterNameAndCurves& TypedParameter : TransformParameters)
				{
					const FName& Name = TypedParameter.ParameterName;
					FRigControlElement* ControlElement = ControlRig->FindControl(Name);
					if (ControlElement && ControlElement->Settings.ControlType == ERigControlType::EulerTransform)
					{
						FEulerTransform EulerTransform(FEulerTransform::Identity);
					
						const float Alpha = BoneSampleTime.GetSubFrame();
						auto EvaluateToTransform = [&TypedParameter](const FFrameNumber& Frame, FTransform& InOutTransform, TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::FTimeEvaluationCache* Cache)
						{
							auto EvaluateValue = [Frame, Cache](const auto& Channel, auto& Target)
							{
								if (Channel.GetDefault().IsSet())
								{
									Target = Channel.GetDefault().GetValue();
								}
								else
								{
									auto Value = (float)Target;
									TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::EvaluateWithCache(&Channel, Cache, Frame, Value);
									Target = Value;
								}
							};

							auto EvaluateVector = [EvaluateValue](const auto& VectorChannels, auto& TargetVector)
							{
								EvaluateValue(VectorChannels[0], TargetVector[0]);
								EvaluateValue(VectorChannels[1], TargetVector[1]);
								EvaluateValue(VectorChannels[2], TargetVector[2]);
							};

							FVector Location(FVector::ZeroVector), Scale(FVector::OneVector);
							EvaluateVector(TypedParameter.Translation, Location);
							InOutTransform.SetTranslation(Location);
							EvaluateVector(TypedParameter.Scale, Scale);
							InOutTransform.SetScale3D(Scale);

							FRotator Rotator;
							EvaluateValue(TypedParameter.Rotation[0], Rotator.Roll);
							EvaluateValue(TypedParameter.Rotation[1], Rotator.Pitch);
							EvaluateValue(TypedParameter.Rotation[2], Rotator.Yaw);
						
							InOutTransform.SetRotation(Rotator.Quaternion());
						};

						auto ExtractTransform = [&TypedParameter, NumberOfKeys](const FFrameNumber& Frame, FEulerTransform& InOutEulerTransform, TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::FTimeEvaluationCache* Cache)
						{
							auto ExtractValue = [&TypedParameter, Frame, Cache, NumberOfKeys](const auto& Channel, auto& Target)
							{
								if (Channel.HasAnyData())
								{
									const int32 NumValues = Channel.GetValues().Num();
									// No keys, but has data so DefaultValue is set
									if (NumValues == 0)
									{
										Target = Channel.GetDefault().GetValue();
									}
									// Uniform keys
									else if (NumValues == NumberOfKeys)
									{
										Target = Channel.GetValues()[Frame.Value].Value;
									}
									// Non-uniform keys
									else
									{
										auto Value = (float)Target;
										TMovieSceneCurveChannelImpl<FMovieSceneFloatChannel>::EvaluateWithCache(&Channel, Cache, Frame, Value);
										Target = Value;
									}
								}
							};

							auto ExtractVector = [ExtractValue](const auto& VectorChannels, auto& TargetVector)
							{
								ExtractValue(VectorChannels[0], TargetVector[0]);
								ExtractValue(VectorChannels[1], TargetVector[1]);
								ExtractValue(VectorChannels[2], TargetVector[2]);
							};

							ExtractVector(TypedParameter.Translation, InOutEulerTransform.Location);
							ExtractVector(TypedParameter.Scale, InOutEulerTransform.Scale);

							ExtractValue(TypedParameter.Rotation[0], InOutEulerTransform.Rotation.Roll);
							ExtractValue(TypedParameter.Rotation[1], InOutEulerTransform.Rotation.Pitch);
							ExtractValue(TypedParameter.Rotation[2], InOutEulerTransform.Rotation.Yaw);
						};
						// Assume no interpolation due to uniform keys
						if (FMath::IsNearlyZero(Alpha))
						{
							if (EvaluationContext.InterpolationType == EAnimInterpolationType::Linear)
							{
								ExtractTransform(BoneSampleTime.FrameNumber, EulerTransform, &FromFrameTimeEvaluationCache);
							}
							else if (EvaluationContext.InterpolationType == EAnimInterpolationType::Step)
							{
								ExtractTransform(BoneSampleTime.FrameNumber, EulerTransform, &FromFrameTimeEvaluationCache);
							}
						}
						// Interpolate between two uniform keys
						else
						{
							const FFrameNumber FromFrame = BoneSampleTime.FloorToFrame();
							const FFrameNumber ToFrame = BoneSampleTime.CeilToFrame();

							FTransform FromBoneTransform;
							EvaluateToTransform(FromFrame, FromBoneTransform, &FromFrameTimeEvaluationCache);
							FTransform ToBoneTransform;
							EvaluateToTransform(ToFrame, ToBoneTransform, &ToFrameTimeEvaluationCache);

							FTransform FinalTransform;
							FinalTransform.Blend(FromBoneTransform, ToBoneTransform, Alpha);
						
							EulerTransform = FEulerTransform(FinalTransform);
						}
						RigHierarchy->SetControlValue(ControlElement, FRigControlValue::Make<FRigControlValue::FEulerTransform_Float>(EulerTransform), ERigControlValueType::Current, false, true, false, false);
					}
				}
			}
		}

		FKRigSection->SetDoNotKey(bWasDoNotKey);
	}
}

FTransformCurve* UAnimationSequencerDataModel::FindMutableTransformCurveById(const FAnimationCurveIdentifier& CurveIdentifier)
{
	for (FTransformCurve& TransformCurve : LegacyCurveData.TransformCurves)
	{
		if (TransformCurve.GetName() == CurveIdentifier.CurveName)
		{
			return &TransformCurve;
		}
	}

	return nullptr;
}

FFloatCurve* UAnimationSequencerDataModel::FindMutableFloatCurveById(const FAnimationCurveIdentifier& CurveIdentifier)
{
	for (FFloatCurve& FloatCurve : LegacyCurveData.FloatCurves)
	{
		if (FloatCurve.GetName() == CurveIdentifier.CurveName)
		{
			return &FloatCurve;
		}
	}

	return nullptr;
}

FAnimCurveBase* UAnimationSequencerDataModel::FindMutableCurveById(const FAnimationCurveIdentifier& CurveIdentifier)
{
	switch (CurveIdentifier.CurveType)
	{
	case ERawCurveTrackTypes::RCT_Float:
		return FindMutableFloatCurveById(CurveIdentifier);
	case ERawCurveTrackTypes::RCT_Transform:
		return FindMutableTransformCurveById(CurveIdentifier);
	default:
		checkf(false, TEXT("Invalid curve identifier type"));
	}

	return nullptr;
}

FRichCurve* UAnimationSequencerDataModel::GetMutableRichCurve(const FAnimationCurveIdentifier& CurveIdentifier)
{
	FRichCurve* RichCurve = nullptr;

	if (CurveIdentifier.IsValid())
	{
		if (CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Float)
		{
			FFloatCurve* Curve = FindMutableFloatCurveById(CurveIdentifier);
			if (Curve)
			{
				RichCurve = &Curve->FloatCurve;
			}
		}
		else if (CurveIdentifier.CurveType == ERawCurveTrackTypes::RCT_Transform)
		{
			if (CurveIdentifier.Channel != ETransformCurveChannel::Invalid && CurveIdentifier.Axis != EVectorCurveChannel::Invalid)
			{
				// Dealing with transform curve
				if (FTransformCurve* TransformCurve = FindMutableTransformCurveById(CurveIdentifier))
				{
					if (FVectorCurve* VectorCurve = TransformCurve->GetVectorCurveByIndex(static_cast<int32>(CurveIdentifier.Channel)))
					{
						RichCurve = &VectorCurve->FloatCurves[static_cast<int32>(CurveIdentifier.Axis)];
					}
				}

			}
		}
	}

	return RichCurve;
}

void UAnimationSequencerDataModel::IterateBoneKeys(const FName& BoneName, TFunction<bool(const FVector3f& Pos, const FQuat4f&, const FVector3f, const FFrameNumber&)> IterationFunction) const
{
	ValidateSequencerData();
	ValidateControlRigData();

	if (const UMovieSceneControlRigParameterSection* Section = GetFKControlRigSection())
	{
		if (const URigHierarchy* Hierarchy = GetControlRigHierarchy())
		{
			const FRigElementKey BoneControlKey(UFKControlRig::GetControlName(BoneName, ERigElementType::Bone), ERigElementType::Control);
			if (const FTransformParameterNameAndCurves* ControlCurvePtr = Section->GetTransformParameterNamesAndCurves().FindByPredicate([CurveName = BoneControlKey.Name](const FTransformParameterNameAndCurves& TransformParameter)
				{
					return TransformParameter.ParameterName == CurveName;
				}))
			{
				const FTransformParameterNameAndCurves& ControlCurve = *ControlCurvePtr;

				struct FChannelInfo
				{
					bool bConstant = true;
					bool bUniform = true;
				};
				FChannelInfo PosChannels[3];
				FChannelInfo RotChannels[3];
				FChannelInfo ScaleChannels[3];

				const int32 NumberOfKeys = GetNumberOfKeys();

				int32 MaxNumberOfKeys = 1;
				TSet<FFrameNumber> FrameNumbers;
				for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
				{
					auto ProcessChannel = [&MaxNumberOfKeys, NumberOfKeys](const FMovieSceneFloatChannel& Channel, FChannelInfo& Info)
						{
							if (Channel.HasAnyData())
							{
								if (Channel.GetNumKeys() == 0)
								{
									Info.bUniform = false;
								}
								else
								{
									Info.bConstant = false;
									if (Channel.GetNumKeys() != NumberOfKeys)
									{
										Info.bUniform = false;
									}

									MaxNumberOfKeys = FMath::Max(MaxNumberOfKeys, Channel.GetNumKeys());
								}
							}
						};

					ProcessChannel(ControlCurve.Translation[ChannelIndex], PosChannels[ChannelIndex]);
					ProcessChannel(ControlCurve.Rotation[ChannelIndex], RotChannels[ChannelIndex]);
					ProcessChannel(ControlCurve.Scale[ChannelIndex], ScaleChannels[ChannelIndex]);
				}


				const int32 NumberOfKeysToIterate = MaxNumberOfKeys;
				const FRigControlElement* BoneControl = Hierarchy->Find<FRigControlElement>(BoneControlKey);
				const FTransform InitialTransform = BoneControl ? BoneControl->GetTransform().Initial.Local.Get() : FTransform::Identity;

				// Initialize components with initial control values (in case there is no default value nor keys)
				FVector3f PreviousPos = FVector3f(InitialTransform.GetLocation());
				FVector3f PreviousScale = FVector3f(InitialTransform.GetScale3D());
				FVector3f PreviousRot = FVector3f(InitialTransform.GetRotation().Euler());
				FQuat4f PreviousQuat;

				for (int32 KeyIndex = 0; KeyIndex < NumberOfKeysToIterate; ++KeyIndex)
				{
					const FFrameNumber Frame = KeyIndex;

					for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
					{
						if (PosChannels[ChannelIndex].bConstant)
						{
							if (KeyIndex == 0)
							{
								PreviousPos[ChannelIndex] = ControlCurve.Translation[ChannelIndex].GetDefault().Get(PreviousPos[ChannelIndex]);
							}
						}
						else
						{
							ControlCurve.Translation[ChannelIndex].Evaluate(Frame, PreviousPos[ChannelIndex]);
						}

						if (RotChannels[ChannelIndex].bConstant)
						{
							if (KeyIndex == 0)
							{
								PreviousRot[ChannelIndex] = ControlCurve.Rotation[ChannelIndex].GetDefault().Get(PreviousRot[ChannelIndex]);
							}
						}
						else
						{
							ControlCurve.Rotation[ChannelIndex].Evaluate(Frame, PreviousRot[ChannelIndex]);
						}

						if (ScaleChannels[ChannelIndex].bConstant)
						{
							if (KeyIndex == 0)
							{
								PreviousScale[ChannelIndex] = ControlCurve.Scale[ChannelIndex].GetDefault().Get(PreviousScale[ChannelIndex]);
							}
						}
						else
						{
							ControlCurve.Scale[ChannelIndex].Evaluate(Frame, PreviousScale[ChannelIndex]);
						}
					}

					PreviousQuat = FQuat4f::MakeFromEuler(PreviousRot);
					if (!IterationFunction(PreviousPos, PreviousQuat, PreviousScale, Frame))
					{
						return;
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE //"AnimSequencerDataModel"
