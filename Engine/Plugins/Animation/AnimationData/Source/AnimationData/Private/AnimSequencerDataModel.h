// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequence.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Sections/MovieSceneParameterSection.h"
#include "Animation/AnimData/AnimDataModelNotifyCollector.h"

#include "AnimSequencerDataModel.generated.h"

#define UE_API ANIMATIONDATA_API

class UControlRig;
class UMovieSceneControlRigParameterTrack;
class UMovieSceneControlRigParameterSection;
class UFKControlRig;
class URigHierarchy;

USTRUCT()
struct FAnimationCurveMetaData
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Flags = 0;

	UPROPERTY()
	FLinearColor Color = FLinearColor::White;

	UPROPERTY()
	FString Comment;
};

UCLASS(MinimalAPI)
class UAnimationSequencerDataModel : public UMovieSceneSequence, public IAnimationDataModel
{
private:
	GENERATED_BODY()
public:
	static UE_API int32 RetainFloatCurves;
	static UE_API int32 ValidationMode;
	static UE_API int32 UseDirectFKControlRigMode;
	static UE_API int32 LazyRigHierarchyInitializationMode;
	
	/** Begin UObject overrides*/
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual bool IsEditorOnly() const override { return true; }
#if WITH_EDITOR
	UE_API virtual void WillNeverCacheCookedPlatformDataAgain() override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
#endif
	/** End UObject overrides */

	/** Begin UMovieSceneSequence overrides */
	virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) override {}
	virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const override { return false; }
	virtual void LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override {}
	UE_API virtual UMovieScene* GetMovieScene() const override;
	UE_API virtual UObject* GetParentObject(UObject* Object) const override;
	virtual void UnbindPossessableObjects(const FGuid& ObjectId) override {}
	virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* Context) override {}
	virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* Context) override {}
	UE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	/** End UMovieSceneSequence overrides */
	
	/** Begin IAnimationDataModel overrides*/
	UE_API virtual double GetPlayLength() const override;
	UE_API virtual int32 GetNumberOfFrames() const override;	
	UE_API virtual int32 GetNumberOfKeys() const override;
	UE_API virtual FFrameRate GetFrameRate() const override;
	UE_API PRAGMA_DISABLE_DEPRECATION_WARNINGS	
	virtual const TArray<FBoneAnimationTrack>& GetBoneAnimationTracks() const override;
	UE_API virtual const FBoneAnimationTrack& GetBoneTrackByIndex(int32 TrackIndex) const override;
	UE_API virtual const FBoneAnimationTrack& GetBoneTrackByName(FName TrackName) const override;
	UE_API virtual int32 GetBoneTrackIndex(const FBoneAnimationTrack& Track) const override;
	UE_API virtual int32 GetBoneTrackIndexByName(FName TrackName) const override;	
	UE_API virtual bool IsValidBoneTrackIndex(int32 TrackIndex) const override;
	UE_API virtual int32 GetNumBoneTracks() const override;
	UE_API virtual void GetBoneTrackNames(TArray<FName>& OutNames) const override;
	UE_API virtual const FAnimCurveBase* FindCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	UE_API virtual const FFloatCurve* FindFloatCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	UE_API virtual const FTransformCurve* FindTransformCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	UE_API virtual const FRichCurve* FindRichCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual bool IsValidBoneTrackName(const FName& TrackName) const override;
	UE_API virtual FTransform GetBoneTrackTransform(FName TrackName, const FFrameNumber& FrameNumber) const override;
	UE_API virtual void GetBoneTrackTransforms(FName TrackName, const TArray<FFrameNumber>& FrameNumbers, TArray<FTransform>& OutTransforms) const override;	
	UE_API virtual void GetBoneTrackTransforms(FName TrackName, TArray<FTransform>& OutTransforms) const override;
	UE_API virtual void GetBoneTracksTransform(const TArray<FName>& TrackNames, const FFrameNumber& FrameNumber, TArray<FTransform>& OutTransforms) const override;	
	UE_API virtual FTransform EvaluateBoneTrackTransform(FName TrackName, const FFrameTime& FrameTime, const EAnimInterpolationType& Interpolation) const override;	
	UE_API virtual const FAnimationCurveData& GetCurveData() const override;
	UE_API virtual int32 GetNumberOfTransformCurves() const override;
	UE_API virtual int32 GetNumberOfFloatCurves() const override;
	UE_API virtual const TArray<struct FFloatCurve>& GetFloatCurves() const override;
	UE_API virtual const TArray<struct FTransformCurve>& GetTransformCurves() const override;
	UE_API virtual const FAnimCurveBase& GetCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	UE_API virtual const FFloatCurve& GetFloatCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	UE_API virtual const FTransformCurve& GetTransformCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	UE_API virtual const FRichCurve& GetRichCurve(const FAnimationCurveIdentifier& CurveIdentifier) const override;
	UE_API virtual TArrayView<const FAnimatedBoneAttribute> GetAttributes() const override;
	UE_API virtual int32 GetNumberOfAttributes() const override;
	UE_API virtual int32 GetNumberOfAttributesForBoneIndex(const int32 BoneIndex) const override;
	UE_API virtual void GetAttributesForBone(const FName& BoneName, TArray<const FAnimatedBoneAttribute*>& OutBoneAttributes) const override;
	UE_API virtual const FAnimatedBoneAttribute& GetAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier) const override;
	UE_API virtual const FAnimatedBoneAttribute* FindAttribute(const FAnimationAttributeIdentifier& AttributeIdentifier) const override;
	UE_API virtual UAnimSequence* GetAnimationSequence() const override;
	virtual FAnimDataModelModifiedEvent& GetModifiedEvent() override { return ModifiedEvent; }
	UE_API virtual FGuid GenerateGuid(const FGuidGenerationSettings& InSettings) const override;
#if WITH_EDITOR
	UE_API virtual FString GenerateDebugStateString() const override;
#endif
	UE_API virtual TScriptInterface<IAnimationDataController> GetController() override;
	UE_API virtual IAnimationDataModel::FModelNotifier& GetNotifier() override;
	virtual FAnimDataModelModifiedDynamicEvent& GetModifiedDynamicEvent() override { return ModifiedEventDynamic; }
    UE_API virtual void Evaluate(FAnimationPoseData& InOutPoseData, const UE::Anim::DataModel::FEvaluationContext& EvaluationContext) const override;
	virtual bool HasBeenPopulated() const override { return bPopulated; }
	UE_API virtual void IterateBoneKeys(const FName& BoneName, TFunction<bool(const FVector3f& Pos, const FQuat4f&, const FVector3f, const FFrameNumber&)> IterationFunction) const override;
protected:
	UE_API virtual void OnNotify(const EAnimDataModelNotifyType& NotifyType, const FAnimDataModelNotifPayload& Payload) override final;

	virtual void LockEvaluationAndModification() const override final
	{
		EvaluationLock.Lock();
	}

	virtual bool TryLockEvaluationAndModification() const override final
	{
		return EvaluationLock.TryLock();
	}
	
	virtual void UnlockEvaluationAndModification() const override final
	{
		EvaluationLock.Unlock();
	}

	virtual bool& GetPopulationFlag() override final
	{
		return bPopulated;
	}
	/** End IAnimationDataModel overrides */
	
	/** Controller helper functionality */
	UE_API FTransformCurve* FindMutableTransformCurveById(const FAnimationCurveIdentifier& CurveIdentifier);
	UE_API FFloatCurve* FindMutableFloatCurveById(const FAnimationCurveIdentifier& CurveIdentifier);
	UE_API FAnimCurveBase* FindMutableCurveById(const FAnimationCurveIdentifier& CurveIdentifier);
	UE_API FRichCurve* GetMutableRichCurve(const FAnimationCurveIdentifier& CurveIdentifier);

	UE_API void RegenerateLegacyCurveData();
	UE_API void UpdateLegacyCurveData();

	UE_API void ValidateData() const;
	UE_API void ValidateSequencerData() const;
	UE_API void ValidateControlRigData() const;
	UE_API void ValidateLegacyAgainstControlRigData() const;
	
	UE_API void GeneratePoseData(UControlRig* ControlRig, FAnimationPoseData& InOutPoseData, const UE::Anim::DataModel::FEvaluationContext& EvaluationContext) const;
	UE_API void EvaluateTrack(UMovieSceneControlRigParameterTrack* CR_Track, const UE::Anim::DataModel::FEvaluationContext& EvaluationContext) const;

	UE_API UMovieSceneControlRigParameterTrack* GetControlRigTrack() const;
	UE_API UMovieSceneControlRigParameterSection* GetFKControlRigSection() const;
	UE_API URigHierarchy* GetControlRigHierarchy() const;
	UE_API USkeleton* GetSkeleton() const;
	UE_API void InitializeFKControlRig(UFKControlRig* FKControlRig, USkeleton* Skeleton, bool bForceHierarchyInitialization=false) const;
	UE_API UControlRig* GetControlRig() const;
	
	UE_API void IterateTransformControlCurve(const FName& BoneName, TFunction<void(const FTransform&, const FFrameNumber&)> IterationFunction, const TArray<FFrameNumber>* InFrameNumbers = nullptr) const;
	UE_API void GenerateTransformKeysForControl(const FName& BoneName, TArray<FTransform>& InOutTransforms, TArray<FFrameNumber>& InOutFrameNumbers) const;
	UE_API void RemoveOutOfDateControls() const;
	
	UE_API void GenerateTransformKeysForControl(const FName& BoneName, const TArray<FFrameNumber>& FrameNumbers, TArray<FTransform>& InOutTransforms) const;
	UE_API void ClearControlRigData();
private:	
	template <typename HasherType>
	void GenerateStateHash(HasherType& Hasher, const FGuidGenerationSettings& InSettings) const;

	UE_API void InitializeRigHierarchy(UFKControlRig* FKControlRig, const USkeleton* Skeleton) const;
	
	/** Dynamic delegate event allows scripting to register to any broadcast-ed notify. */
	UPROPERTY(BlueprintAssignable, Transient, Category = AnimationDataModel, meta = (ScriptName = "ModifiedEvent", AllowPrivateAccess = "true"))
	FAnimDataModelModifiedDynamicEvent ModifiedEventDynamic;	
	FAnimDataModelModifiedEvent ModifiedEvent;
	TUniquePtr<IAnimationDataModel::FModelNotifier> Notifier;
	UE::Anim::FAnimDataModelNotifyCollector Collector;

	UPROPERTY(VisibleAnywhere, Category=AnimSequencer)
	FAnimationCurveData LegacyCurveData;

	UPROPERTY(VisibleAnywhere, Category=AnimSequencer)
	TArray<FAnimatedBoneAttribute> AnimatedBoneAttributes;

	// Movie scene instance containing FK Control rig and section representing the animation data
	UPROPERTY(VisibleAnywhere, Category=AnimSequencer)
	TObjectPtr<UMovieScene> MovieScene = nullptr;

	// Per-curve information holding flags/color, due to be deprecated in the future
	UPROPERTY(VisibleAnywhere, Category=AnimSequencer)
	TMap<FAnimationCurveIdentifier, FAnimationCurveMetaData> CurveIdentifierToMetaData;

	UPROPERTY(VisibleAnywhere, Category=AnimSequencer)
	bool bPopulated = false;

	// Raw data GUID taken from UAnimSequence when initially populating - this allows for retaining compressed data state initially
	UPROPERTY(VisibleAnywhere, Category=AnimSequencer)
	FGuid CachedRawDataGUID;

	UPROPERTY(VisibleAnywhere, Transient, Category=AnimSequencer)
	mutable bool bRigHierarchyInitialized = false;

	// Scope lock to prevent contention around ControlRig->Evaluation()
	mutable FCriticalSection EvaluationLock;

	friend class UAnimSequencerController;
};


#undef UE_API
