// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequencerHelpers.h"

#include "Channels/MovieSceneFloatChannel.h"
#include "Curves/RichCurve.h"
#include "Misc/FrameRate.h"
#include "Algo/Transform.h"
#include "Sections/MovieSceneParameterSection.h"

void AnimSequencerHelpers::ConvertFloatChannelToRichCurve(const FMovieSceneFloatChannel& Channel, FRichCurve& OutCurve, const FFrameRate& TargetFrameRate)
{
	const TArrayView<const FFrameNumber> Times = Channel.GetTimes();
	const TArrayView<const FMovieSceneFloatValue> Values = Channel.GetValues();

	const int32 NumKeys = Channel.GetNumKeys();
	OutCurve.Keys.SetNum(NumKeys);

	OutCurve.PreInfinityExtrap = Channel.PreInfinityExtrap;
	OutCurve.PostInfinityExtrap = Channel.PostInfinityExtrap;

	for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
	{
		// Convert from channel to requested frame-rate
		const FFrameNumber KeyFrameNumber = Times[KeyIndex];
		const double KeyTime = Channel.GetTickResolution().AsSeconds(KeyFrameNumber);

		OutCurve.Keys[KeyIndex].Time = static_cast<float>(KeyTime);

		const FFrameNumber* PrevKey = KeyIndex > 0 ? &Times[KeyIndex - 1] : nullptr;
		const FFrameNumber* NextKey = KeyIndex < (NumKeys - 1) ? &Times[KeyIndex + 1] : nullptr;

		const int32 FrameNumberDelta = [&]() -> int32
		{
			if (PrevKey && NextKey)
			{
				return NextKey->Value - PrevKey->Value;
			}
			else if (PrevKey)
			{
				return Times[KeyIndex].Value - PrevKey->Value;
			}
			else if (NextKey)
			{
				return NextKey->Value - Times[KeyIndex].Value;
			}

			return 1;
		}();
		
		const double SecondsDelta = [&]() -> double
		{
			if (PrevKey && NextKey)
			{
				return Channel.GetTickResolution().AsSeconds(*NextKey) - Channel.GetTickResolution().AsSeconds(*PrevKey);
			}
			else if (PrevKey)
			{
				return KeyTime - Channel.GetTickResolution().AsSeconds(*PrevKey);
			}
			else if (NextKey)
			{
				return Channel.GetTickResolution().AsSeconds(*NextKey) - KeyTime;
			}

			return 1.0;
		}();		

		// Ratio between rich-curve and moviescene key(s) timing, if there are any surrounding keys (otherwise default to a ratio of 1:1)
		const double KeyTimingRatio = (PrevKey || NextKey) ?  FrameNumberDelta/SecondsDelta : 1.0;
				
		ConvertFloatValueToRichCurveKey(Values[KeyIndex], OutCurve.Keys[KeyIndex], KeyTimingRatio);
	}
}

void AnimSequencerHelpers::ConvertRichCurveKeysToFloatChannel(const TArray<FRichCurveKey>& CurveKeys, FMovieSceneFloatChannel& OutChannel)
{
	TArray<FFrameNumber> Times;
	TArray<FMovieSceneFloatValue> Values;
	
	const int32 NumCurveKeys = CurveKeys.Num();
	Times.Reserve(NumCurveKeys);
	Values.Reserve(NumCurveKeys);
	
	const FFrameRate& TargetFrameRate = OutChannel.GetTickResolution();
	
	for (int32 KeyIndex = 0; KeyIndex < NumCurveKeys; ++KeyIndex)
	{
		const FRichCurveKey* PrevKey = KeyIndex > 0 ? &CurveKeys[KeyIndex - 1] : nullptr;
		const FRichCurveKey* NextKey = KeyIndex < (NumCurveKeys - 1) ? &CurveKeys[KeyIndex + 1] : nullptr;
		const FRichCurveKey& RichCurveKey = CurveKeys[KeyIndex];

		const float SecondsDelta = [&]() -> float
		{
			if (PrevKey && NextKey)
			{
				return NextKey->Time - PrevKey->Time;
			}
			else if (PrevKey)
			{
				return RichCurveKey.Time - PrevKey->Time;
			}
			else if (NextKey)
			{
				return NextKey->Time - RichCurveKey.Time;
			}

			return 1.f;
		}();
		
		const int32 FrameNumberDelta = [&]() -> int32
		{
			if (PrevKey && NextKey)
			{
				return TargetFrameRate.AsFrameTime(NextKey->Time).RoundToFrame().Value - TargetFrameRate.AsFrameTime(PrevKey->Time).RoundToFrame().Value;
			}
			else if (PrevKey)
			{
				return TargetFrameRate.AsFrameTime(RichCurveKey.Time).RoundToFrame().Value - TargetFrameRate.AsFrameTime(PrevKey->Time).RoundToFrame().Value;
			}
			else if (NextKey)
			{
				return TargetFrameRate.AsFrameTime(NextKey->Time).RoundToFrame().Value - TargetFrameRate.AsFrameTime(RichCurveKey.Time).RoundToFrame().Value;
			}

			return 1;
		}();
		
		Times.Add(TargetFrameRate.AsFrameTime(RichCurveKey.Time).RoundToFrame());
		FMovieSceneFloatValue& Value = Values.AddDefaulted_GetRef();

		// Ratio between rich-curve and moviescene key(s) timing, if there are any surrounding keys (otherwise default to a ratio of 1:1)
		const double KeyTimingRatio = (PrevKey || NextKey) ? SecondsDelta / FrameNumberDelta : 1.0;
		
		ConvertRichCurveKeyToFloatValue(RichCurveKey, Value, KeyTimingRatio, TargetFrameRate.AsInterval());
	}

	// Need to go through and generate tangents for linear keys residing next to cubic keys
	// as MovieSceneChannel considers linear keys as cubic keys with linear tangents
	for (int32 KeyIndex = 0; KeyIndex < NumCurveKeys; ++KeyIndex)
	{
		FMovieSceneFloatValue& Key = Values[KeyIndex];
		if (Key.InterpMode == ERichCurveInterpMode::RCIM_Linear)
		{
			Key.Tangent.TangentWeightMode = RCTWM_WeightedNone; 
			if (KeyIndex > 0)
			{
				// Always zero out tangent here to keep behaviour consistent with FRichCurve evaluation
				Key.Tangent.ArriveTangent = 0.f;
			}

			if (KeyIndex < (NumCurveKeys - 1))
			{
				const FMovieSceneFloatValue& NextKey = Values[KeyIndex + 1];
				const double NextTimeDiff = FMath::Max<double>(KINDA_SMALL_NUMBER, Times[KeyIndex + 1].Value - Times[KeyIndex].Value);

				// Generate linear tangent (delta value / delta time)
				const double NewTangent = (NextKey.Value - Key.Value) / NextTimeDiff;
				Key.Tangent.LeaveTangent = static_cast<float>(NewTangent);
			}
		}
		// Also need to generate linear tangents for cubic keys which are to the right of linear keys
		else if (Key.InterpMode == ERichCurveInterpMode::RCIM_Cubic)
		{
			if (KeyIndex > 0)
			{
				const FMovieSceneFloatValue& PreviousKey = Values[KeyIndex - 1];
				if (PreviousKey.InterpMode == ERichCurveInterpMode::RCIM_Linear)
				{
					// Retain weighted tangent mode if any
					Key.Tangent.TangentWeightMode = Key.Tangent.TangentWeightMode == RCTWM_WeightedBoth || Key.Tangent.TangentWeightMode == RCTWM_WeightedLeave ? RCTWM_WeightedLeave : RCTWM_WeightedNone;
					// Set tangent mode to break to allow linear arrive tangent while keeping leave tangent
					Key.TangentMode = RCTM_Break;
					
					const double PrevTimeDiff = FMath::Max<double>(KINDA_SMALL_NUMBER, Times[KeyIndex].Value - Times[KeyIndex - 1].Value);
					// Generate linear tangent (delta value / delta time)
					Key.Tangent.ArriveTangent = static_cast<float>((Key.Value - PreviousKey.Value) / PrevTimeDiff);
				}
			}
		}
	}
	
	OutChannel.SetKeysOnly(Times, Values);
}

void AnimSequencerHelpers::ConvertRichCurveKeyToFloatValue(const FRichCurveKey& RichCurveKey, FMovieSceneFloatValue& OutMovieSceneKey, double TangentRatio /*= 1.0*/, double SecondsPerFrame /*= 1.0*/)
{
	OutMovieSceneKey.Value = RichCurveKey.Value;
	
	OutMovieSceneKey.Tangent.TangentWeightMode = RichCurveKey.TangentWeightMode;
	if( OutMovieSceneKey.Tangent.TangentWeightMode != RCTWM_WeightedNone &&
			OutMovieSceneKey.Tangent.TangentWeightMode != RCTWM_WeightedArrive &&
			OutMovieSceneKey.Tangent.TangentWeightMode != RCTWM_WeightedLeave &&
			OutMovieSceneKey.Tangent.TangentWeightMode != RCTWM_WeightedBoth
		)
	{
		OutMovieSceneKey.Tangent.TangentWeightMode = RCTWM_WeightedNone;
	}
		
	OutMovieSceneKey.Tangent.ArriveTangentWeight = RichCurveKey.ArriveTangentWeight;
	OutMovieSceneKey.Tangent.LeaveTangentWeight = RichCurveKey.LeaveTangentWeight;

	if (OutMovieSceneKey.Tangent.TangentWeightMode == RCTWM_WeightedNone)
	{
		OutMovieSceneKey.Tangent.ArriveTangent = static_cast<float>(RichCurveKey.ArriveTangent * TangentRatio);
		OutMovieSceneKey.Tangent.LeaveTangent = static_cast<float>(RichCurveKey.LeaveTangent * TangentRatio);
	}
	else
	{		
		OutMovieSceneKey.Tangent.ArriveTangent = static_cast<float>(RichCurveKey.ArriveTangent * SecondsPerFrame);
		OutMovieSceneKey.Tangent.LeaveTangent = static_cast<float>(RichCurveKey.LeaveTangent * SecondsPerFrame);
	}		
			
	OutMovieSceneKey.TangentMode = RichCurveKey.TangentMode;
	OutMovieSceneKey.InterpMode = RichCurveKey.InterpMode;
}

void AnimSequencerHelpers::ConvertFloatValueToRichCurveKey(const FMovieSceneFloatValue& MovieSceneKey, FRichCurveKey& OutRichCurveKey, double TangentRatio /*= 1.0*/)
{
	OutRichCurveKey.Value = MovieSceneKey.Value;
	OutRichCurveKey.InterpMode = MovieSceneKey.InterpMode;	
	
	OutRichCurveKey.TangentMode = MovieSceneKey.TangentMode;
	OutRichCurveKey.ArriveTangent = static_cast<float>(MovieSceneKey.Tangent.ArriveTangent * TangentRatio);
	OutRichCurveKey.LeaveTangent = static_cast<float>(MovieSceneKey.Tangent.LeaveTangent * TangentRatio);
		
	OutRichCurveKey.TangentWeightMode = MovieSceneKey.Tangent.TangentWeightMode;
	OutRichCurveKey.ArriveTangentWeight = MovieSceneKey.Tangent.ArriveTangentWeight;
	OutRichCurveKey.LeaveTangentWeight = MovieSceneKey.Tangent.LeaveTangentWeight;				
}

void AnimSequencerHelpers::GenerateUniqueFramesForTransformCurve(const FTransformParameterNameAndCurves& TransformParameter, TSet<FFrameNumber>& InOutFrames)
{
	for (int32 ChannelIndex = 0; ChannelIndex < 3; ++ChannelIndex)
	{
		Algo::Transform(TransformParameter.Translation[ChannelIndex].GetTimes(), InOutFrames, [](FFrameNumber Number) { return Number; }); 
		Algo::Transform(TransformParameter.Rotation[ChannelIndex].GetTimes(), InOutFrames, [](FFrameNumber Number) { return Number; });
		Algo::Transform(TransformParameter.Scale[ChannelIndex].GetTimes(), InOutFrames, [](FFrameNumber Number) { return Number; });
	}
}