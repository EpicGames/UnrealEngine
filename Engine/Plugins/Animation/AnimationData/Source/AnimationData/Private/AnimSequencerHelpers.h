// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Misc/FrameNumber.h"

struct FMovieSceneFloatValue;
struct FRichCurveKey;
struct FTransformParameterNameAndCurves;
struct FMovieSceneFloatChannel;
struct FRichCurve;
struct FFrameRate;

struct AnimSequencerHelpers
{
	// Converts an individual rich-curve key to its sequencer (moviescene channel) equivalent
	static void ConvertRichCurveKeyToFloatValue(const FRichCurveKey& RichCurveKey, FMovieSceneFloatValue& OutMovieSceneKey, double TangentRatio = 1.0, double SecondsPerFrame = 1.0);
	// Converts all keys from a FMovieSceneFloatChannel to its RichCurve equivalent, converting the source (moviescene) frame numbers to the target seconds-based representation
	static void ConvertFloatChannelToRichCurve(const FMovieSceneFloatChannel& Channel, FRichCurve& OutCurve, const FFrameRate& TargetFrameRate);
	// Converts all keys from a FRichCurveKey to its sequencer (moviescene equivalent), converting the source seconds-based time representation to (moviescene channel) frame numbers
	static void ConvertRichCurveKeysToFloatChannel(const TArray<FRichCurveKey>& CurveKeys, FMovieSceneFloatChannel& OutChannel);
	// Generates unique frame numbers found across all channels of a (moviescene) transform curve
	static void GenerateUniqueFramesForTransformCurve(const FTransformParameterNameAndCurves& TransformParameter, TSet<FFrameNumber>& InOutFrames);
protected:	
	static void ConvertFloatValueToRichCurveKey(const FMovieSceneFloatValue& MovieSceneKey, FRichCurveKey& OutRichCurveKey, double TangentRatio = 1.0);	
};

