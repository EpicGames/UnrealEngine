// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationDataModule.h"
#include "AnimSequencerDataModel.h"
#include "Features/IModularFeatures.h"
#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"

void FAnimationDataModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature( UE::Anim::DataModel::IAnimationDataModels::GetModularFeatureName(),  &SequencerBasedDataModelFeature);
}

void FAnimationDataModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature( UE::Anim::DataModel::IAnimationDataModels::GetModularFeatureName(),  &SequencerBasedDataModelFeature);
}

IMPLEMENT_MODULE(FAnimationDataModule, AnimationData);