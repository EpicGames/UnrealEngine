// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#include "Animation/AnimSequence.h"
#include "AnimSequencerDataModel.h"
#include "Animation/AnimData/IAnimationDataModel.h"

class FSequencerBasedDataModel : public UE::Anim::DataModel::IAnimationDataModels
{
	virtual UClass* GetModelClass(UAnimSequenceBase* OwningAnimationAsset) const override
	{
		checkf(OwningAnimationAsset, TEXT("Trying to retrieve ModelClass for null Animation Asset"));
		if (OwningAnimationAsset->GetClass()->IsChildOf(UAnimSequence::StaticClass()))
		{
			// Only return a Sequencer based model for AnimationSequences at the moment
			return UAnimationSequencerDataModel::StaticClass();
		}

		return nullptr;
	}
};

class FAnimationDataModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
protected:
	FSequencerBasedDataModel SequencerBasedDataModelFeature;
};

